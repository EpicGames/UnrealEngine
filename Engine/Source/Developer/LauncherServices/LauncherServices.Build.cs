// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LauncherServices : ModuleRules
	{
		public LauncherServices(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				});

			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"TargetDeviceServices",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"DesktopPlatform",
					"SessionMessages",
					"TargetPlatform",
					"UnrealEdMessages",
					"Json",
					"TurnkeyIO",
					"Zen"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"DerivedDataCache",
				});
		}
	}
}
