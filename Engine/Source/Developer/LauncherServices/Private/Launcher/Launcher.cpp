// Copyright Epic Games, Inc. All Rights Reserved.

#include "Launcher/Launcher.h"

#include "HAL/RunnableThread.h"
#include "Launcher/LauncherWorker.h"
#include "Profiles/LauncherProfile.h"


/* Static class member instantiations
*****************************************************************************/

FThreadSafeCounter FLauncher::WorkerCounter;


/* ILauncher overrides
 *****************************************************************************/

ILauncherWorkerPtr FLauncher::Launch(const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager, const ILauncherProfileRef& Profile)
{
	if (Profile->IsValidForLaunch())
	{
		TSharedPtr<FLauncherWorker> LauncherWorker = MakeShared<FLauncherWorker>(DeviceProxyManager, Profile, *FString::Printf(TEXT("LauncherWorker%i"), WorkerCounter.Increment()));
		if (LauncherWorker->IsValid())
		{
			FLauncherWorkerStartedDelegate.Broadcast(LauncherWorker, Profile);
			return MoveTemp(LauncherWorker);
		}
	}
	else
	{
		UE_LOGF(LogLauncherProfile, Error, "Launcher profile '%ls' for is not valid for launch.",
			*Profile->GetName());
		for (int32 I = 0; I < (int32)ELauncherProfileValidationErrors::Count; ++I)
		{
			ELauncherProfileValidationErrors::Type Error = (ELauncherProfileValidationErrors::Type)I;
			if (Profile->HasValidationError(Error))
			{
				UE_LOGF(LogLauncherProfile, Error, "ValidationError: %ls", *LexToStringLocalized(Error));
			}
		}
	}

	return nullptr;
}
