// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "ILauncher.h"


/**
 * Implements the game launcher.
 */
class FLauncher
	: public ILauncher
{
public:

	//~ ILauncher Interface

	virtual ILauncherWorkerPtr Launch(const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager, const ILauncherProfileRef& Profile) override;

private:

	/** Worker counter, used to generate unique thread names for each worker. */
	static FThreadSafeCounter WorkerCounter;
};
