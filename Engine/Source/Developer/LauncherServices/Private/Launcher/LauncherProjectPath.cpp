// Copyright Epic Games, Inc. All Rights Reserved.

#include "Launcher/LauncherProjectPath.h"
#include "Misc/Paths.h"

FString FLauncherProjectPath::GetProjectName(const FString& ProjectPath)
{
	if (!ProjectPath.IsEmpty())
	{
		return FPaths::GetBaseFilename(ProjectPath);
	}
	return FString();
}

FString FLauncherProjectPath::GetProjectBasePath(const FString& ProjectPath)
{
	if (!ProjectPath.IsEmpty())
	{
		FString OutPath = ProjectPath;
		FPaths::MakePathRelativeTo(OutPath, *FPaths::RootDir());
		return FPaths::GetPath(OutPath);
	}
	return FString();
}
