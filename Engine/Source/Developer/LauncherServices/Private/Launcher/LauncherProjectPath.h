// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Implements the launcher project path utilities.
 */
class FLauncherProjectPath
{
public:

	/**
	 * Gets the name of the Unreal project to use.
	 */
	static FString GetProjectName(const FString& ProjectPath);

	/**
	 * Gets the base project path for the project (e.g. Samples/Showcases/MyShowcase)
	 */
	static FString GetProjectBasePath(const FString& ProjectPath);

};
