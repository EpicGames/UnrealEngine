// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "ILauncherTask.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "Launcher/LauncherTaskChainState.h"
#include "HAL/RunnableThread.h"

/**
 * Abstract base class for launcher tasks.
 */
class FLauncherTask
	: public FRunnable
	, public ILauncherTask
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InName - The name of the task.
	 * @param InDesc - The description of the task.
	 */
	FLauncherTask(FString InName, FString InDesc)
		: Name{MoveTemp(InName)}
		, Desc{MoveTemp(InDesc)}
	{ }
	
	virtual ~FLauncherTask() override
	{
		if (Thread)
		{
			Thread->WaitForCompletion();
		}
	}

public:

	/**
	 * Adds a task that will execute after this task completed.
	 *
	 * Continuations must be added before this task starts.
	 *
	 * @param Task - The task to add.
	 */
	void AddContinuation( const TSharedPtr<FLauncherTask>& Task )
	{
		if (Status == ELauncherTaskStatus::Pending)
		{
			Continuations.AddUnique(Task);
		}	
	}

	/**
	 * Executes the tasks.
	 *
	 * @param ChainState - State data of the task chain.
	 */
	void Execute(const FLauncherTaskChainState* ChainState)
	{
		check(Status == ELauncherTaskStatus::Pending);

		LocalChainState = ChainState;

		// set this before the thread starts so that way we know what's going on
		FString TaskName(TEXT("FLauncherTask"));
		TaskName.AppendInt(TaskCounter.Increment());
		Status = ELauncherTaskStatus::Busy;
		Thread = TUniquePtr<FRunnableThread>{FRunnableThread::Create(this, *TaskName)};
	}

	/**
	 * Gets the list of tasks to be executed after this task.
	 *
	 * @return Continuation task list.
	 */
	virtual const TArray<TSharedPtr<FLauncherTask> >& GetContinuations( ) const
	{
		return Continuations;
	}

	/**
	 * Checks whether the task chain has finished execution.
	 *
	 * A task chain is finished when this task and all its continuations are finished.
	 *
	 * @return true if the chain is finished, false otherwise.
	 */
	bool IsChainFinished( ) const
	{
		if (IsFinished())
		{
			for (int32 ContinuationIndex = 0; ContinuationIndex < Continuations.Num(); ++ContinuationIndex)
			{
				if (!Continuations[ContinuationIndex]->IsChainFinished())
				{
					return false;
				}
			}

			return true;
		}

		return false;
	}

	bool Succeeded() const
	{
		if (IsChainFinished())
		{
			for (int32 ContinuationIndex = 0; ContinuationIndex < Continuations.Num(); ++ContinuationIndex)
			{
				if (!Continuations[ContinuationIndex]->Succeeded())
				{
					return false;
				}
			}
		}

		return Status == ELauncherTaskStatus::Completed;
	}

	int32 ReturnCode() const override
	{
		if (IsChainFinished())
		{
			for (int32 ContinuationIndex = 0; ContinuationIndex < Continuations.Num(); ++ContinuationIndex)
			{
				if (Continuations[ContinuationIndex]->ReturnCode() != 0)
				{
					return Continuations[ContinuationIndex]->ReturnCode();
				}
			}
		}

		return Result;
	}

public:

	//~ Begin FRunnable Interface

	virtual bool Init( ) override
	{
		return true;
	}

	virtual uint32 Run( ) override
	{
		// this thread has the responsibility of the Status
		// if the status is busy then no other thread can access the status
		check(Status == ELauncherTaskStatus::Busy);

		StartTime = FDateTime::UtcNow();

		FPlatformProcess::Sleep(0.5f);

		TaskStarted.Broadcast(Name);
		bool bSucceeded = PerformTask(*LocalChainState);
		
		if (bSucceeded)
		{
			Status = ELauncherTaskStatus::Completed;
		}
		else if (bCancelling)
		{
			Status = ELauncherTaskStatus::Canceled;
		}
		else
		{
			Status = ELauncherTaskStatus::Failed;
		}

		TaskCompleted.Broadcast(Name);
		if (bSucceeded && !bCancelling)
		{
			ExecuteContinuations();
		}
		else
		{
			CancelContinuations();
		}


		EndTime = FDateTime::UtcNow();
		
		return 0;
	}

	virtual void Stop( ) override
	{
		Cancel();
	}

	virtual void Exit( ) override { }

	//~ End FRunnable Interface

public:

	//~ Begin ILauncherTask Interface

	virtual void Cancel() override
	{
		// this can be called by anyone
		bCancelling = true;

		if (Status == ELauncherTaskStatus::Pending)
		{
			Status = ELauncherTaskStatus::Canceled;
		}
		CancelContinuations();
	}

	virtual FTimespan GetDuration( ) const override
	{
		if (Status == ELauncherTaskStatus::Pending)
		{
			return FTimespan::Zero();
		}

		if (Status == ELauncherTaskStatus::Busy)
		{
			return (FDateTime::UtcNow() - StartTime);
		}

		return (EndTime - StartTime);
	}

	virtual const FString& GetName( ) const override
	{
		return Name;
	}

	virtual const FString& GetDesc( ) const override
	{
		return Desc;
	}

	virtual bool IsCancelling() const override { return bCancelling; }

	virtual ELauncherTaskStatus::Type GetStatus( ) const override
	{
		return Status;
	}

	virtual bool IsFinished( ) const override
	{
		return ((Status == ELauncherTaskStatus::Canceled) ||
				(Status == ELauncherTaskStatus::Completed) ||
				(Status == ELauncherTaskStatus::Failed));
	}

	virtual FOnTaskStartedDelegate& OnStarted() override
	{
		return TaskStarted;
	}

	virtual FOnTaskCompletedDelegate& OnCompleted() override
	{
		return TaskCompleted;
	}

	virtual uint32 GetWarningCount() const override
	{
		return WarningCounter;
	}

	virtual uint32 GetErrorCount() const override
	{
		return ErrorCounter;
	}

	//~ End ILauncherTask Interface

protected:

	/**
	 * Performs the actual task.
	 *
	 * @param ChainState - Holds the state of the task chain.
	 *
	 * @return true if the task completed successfully, false otherwise.
	 */
	virtual bool PerformTask(const FLauncherTaskChainState& ChainState) = 0;

protected:

	/**
	 * Cancels all continuation tasks.
	 */
	void CancelContinuations( ) const
	{
		for (int32 ContinuationIndex = 0; ContinuationIndex < Continuations.Num(); ++ContinuationIndex)
		{
			Continuations[ContinuationIndex]->Cancel();
		}
	}

	/**
	 * Executes all continuation tasks.
	 */
	void ExecuteContinuations( ) const
	{
		for (int32 ContinuationIndex = 0; ContinuationIndex < Continuations.Num(); ++ContinuationIndex)
		{
			Continuations[ContinuationIndex]->Execute(LocalChainState);
		}
	}

private:

	// Holds the tasks that execute after this task completed.
	TArray<TSharedPtr<FLauncherTask> > Continuations;

	// Holds the time at which the task ended.
	FDateTime EndTime;

	// Holds the local state of this task chain.
	const FLauncherTaskChainState* LocalChainState = nullptr;

	// Holds the task's name.
	FString Name;

	// Holds the task's description
	FString Desc;

	// Holds the time at which the task started.
	FDateTime StartTime;

	// Holds the status of this task
	ELauncherTaskStatus::Type Status = ELauncherTaskStatus::Pending;

	// set if this should be cancelled
	bool bCancelling = false;

	// Holds the thread that's running this task.
	TUniquePtr<FRunnableThread> Thread;

	// message delegates
	FOnTaskStartedDelegate TaskStarted;
	FOnTaskCompletedDelegate TaskCompleted;

protected:
	// result
	int32 Result = 0;

	// counters
	uint32 ErrorCounter = 0;
	uint32 WarningCounter = 0;

	// task counter, used to generate unique thread names for each task
	static FThreadSafeCounter TaskCounter;
};
