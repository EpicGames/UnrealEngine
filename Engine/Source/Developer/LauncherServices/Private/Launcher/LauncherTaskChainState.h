// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ILauncherProfile.h"

/**
 * Structure for passing launcher task chain state data.
 */
struct FLauncherTaskChainState
{
	/** The path to the binaries directory (for deployment). */
	FString BinariesDirectory;

	/** The path to the content directory (for deployment). */
	FString ContentDirectory;

	/** The identifier of the deployed application. */
	FString DeployedAppId;

	/** The list of IP addresses and port numbers at which the file server is reachable. */
	FString FileServerAddressListString;

	/** A pointer to the launcher profile. */
	ILauncherProfilePtr Profile;

	/** The session identifier. */
	FGuid SessionId;
};
