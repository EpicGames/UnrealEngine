// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherWorker.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Launcher/LauncherTask.h"
#include "Misc/App.h"

/**
 * class for UAT launcher tasks.
 */
class FLauncherUATTask
	: public FLauncherTask
{
public:

	FLauncherUATTask(FString InCommandLine, FString InName, FString InDesc, UE::HAL::FWriteHandle InWritePipe, UE::HAL::FProcess& InProcess, ILauncherWorker* InWorker, FString InCommandEnd)
		: FLauncherTask{MoveTemp(InName), MoveTemp(InDesc)}
		, CommandLine{MoveTemp(InCommandLine)}
		, WritePipe{InWritePipe}
		, Process{InProcess}
		, CommandText{MoveTemp(InCommandEnd)}
	{
		InWorker->OnOutputReceived().AddRaw(this, &FLauncherUATTask::HandleOutputReceived);
	}

protected:

	/**
	 * Performs the actual task.
	 *
	 * @param ChainState - Holds the state of the task chain.
	 *
	 * @return true if the task completed successfully, false otherwise.
	 */
	virtual bool PerformTask(const FLauncherTaskChainState& ChainState) override
	{
		// spawn a UAT process to cook the data
		// UAT executable path
		FString ExecutablePath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() + TEXTVIEW("Build/BatchFiles"));
		// UAT executable
#if PLATFORM_MAC
		FString Executable = TEXT("RunUAT.command");
#elif PLATFORM_LINUX
		FString Executable = TEXT("RunUAT.sh");
#else
		FString Executable = TEXT("RunUAT.bat");
#endif

		// base UAT command arguments
		FString UATCommandLine;
		FString ProjectPath = *ChainState.Profile->GetProjectPath();
		ProjectPath = FPaths::ConvertRelativePathToFull(ProjectPath);
		UATCommandLine = FString::Printf(TEXT("-ScriptsForProject=\"%s\" -noP4"), *ProjectPath);

		// we expect to pass -nocompile to UAT here as we generally expect UAT to be fully compiled. Besides, installed builds don't even have the source to compile UAT scripts.
		// Only allow UAT to compile scripts dynamically if we pass -development or we have the IsBuildingUAT property set, the latter of which should not allow itself to be set in installed situations.
		bool bAllowCompile = true;
		UATCommandLine += (bAllowCompile && (FParse::Param( FCommandLine::Get(), TEXT("development") ) || ChainState.Profile->IsBuildingUAT())) ? TEXT("") : TEXT(" -nocompile");
		UATCommandLine += TEXT(" -utf8output");

		// specialized command arguments for this particular task
		UATCommandLine += TEXT(" ");
		UATCommandLine += CommandLine;

		// launch UAT and monitor its progress
		Process = UE::HAL::FProcess{{.Uri = *(ExecutablePath / Executable), .Arguments = *UATCommandLine, .WorkingDirectory = *ExecutablePath, .bHidden = true, .StdOut = WritePipe}};

		while (!EndTextFound.load(std::memory_order_relaxed))
		{
			if (!Process.IsRunning())
			{
				Result = *Process.GetExitCode();
				break;
			}

			FPlatformProcess::Sleep(0.25);
		}

		return Result == 0;
	}

	void HandleOutputReceived(const FString& InMessage)
	{
		if (InMessage.Contains(TEXT("Error:"), ESearchCase::IgnoreCase))
		{
			++ErrorCounter;
		}
		else if (InMessage.Contains(TEXT("Warning:"), ESearchCase::IgnoreCase))
		{
			++WarningCounter;
		}

		EndTextFound.store(EndTextFound.load(std::memory_order_relaxed) || InMessage.Contains(CommandText), std::memory_order_relaxed);
	}

private:

	// Command line
	FString CommandLine;

	// write pipe
	UE::HAL::FWriteHandle WritePipe;

	// process
	UE::HAL::FProcess& Process;

	// The end text marker for this UAT command
	FString CommandText;

	std::atomic_bool EndTextFound = false;
};

