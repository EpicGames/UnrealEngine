// Copyright Epic Games, Inc. All Rights Reserved.

#include "Launcher/LauncherWorker.h"
#include "HAL/PlatformTime.h"
#include "HAL/FileManager.h"
#include "ILauncherServicesModule.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "ITurnkeyIOModule.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/Queue.h"
#include "Modules/ModuleManager.h"
#include "Launcher/LauncherTaskChainState.h"
#include "Launcher/LauncherTask.h"
#include "Launcher/LauncherUATTask.h"
#include "PlatformInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Profiles/LauncherProfile.h"
#include "Profiles/LauncherProfileBuildCookRun.h"
#include "DerivedDataCacheInterface.h"


#define LOCTEXT_NAMESPACE "LauncherWorker"


/* Static class member instantiations
*****************************************************************************/

FThreadSafeCounter FLauncherTask::TaskCounter;


/* FLauncherWorker structors
 *****************************************************************************/

FLauncherWorker::FLauncherWorker(const TSharedRef<ITargetDeviceProxyManager>& InDeviceProxyManager, const ILauncherProfileRef& InProfile, const TCHAR* InWorkerName)
	: DeviceProxyManager(InDeviceProxyManager)
	, Profile(InProfile)
{
	CreateAndExecuteTasks(InProfile);
	
	Thread = TUniquePtr<FRunnableThread>{FRunnableThread::Create(this, InWorkerName)};
}


/* FRunnable overrides
 *****************************************************************************/

bool FLauncherWorker::Init( )
{
	return true;
}


uint32 FLauncherWorker::Run( )
{
	LaunchStartTime = FPlatformTime::Seconds();

	{
		auto MessageReceived = [this](FString InMessage)
		{
			InMessage.TrimEndInline();
			FStringView MessageView = InMessage;
			{
				FStringView PackageDevicePrefix = TEXTVIEW("Running Package@Device:");
				if (MessageView.StartsWith(PackageDevicePrefix))
				{
					FStringView Value = MessageView.RightChop(PackageDevicePrefix.Len());
					int32 SplitIndex;
					if (Value.FindChar('@', SplitIndex))
					{
						FString Package(Value.SubStr(0, SplitIndex));
						FString Device(Value.SubStr(SplitIndex + 1, Value.Len()));
						AddDevicePackagePair(Device, Package);
					}
				}
			}
			OutputMessageReceived.Broadcast(InMessage);
		};

		FString Line;
		auto ReadLines = [this, &MessageReceived, &Line]
		{
			FString NewLine = InputPipe.Read();
			if (NewLine.IsEmpty())
			{
				return false;
			}

			// process the string to break it up into lines
			Line += NewLine;
	
			if (int32 NewLineIndex; NewLine.FindChar(TCHAR('\n'), NewLineIndex))
			{
				FStringView NewLineView = FStringView{NewLine}.RightChop(NewLineIndex + 1);
				MessageReceived(Line.LeftChop(NewLineView.Len() + 1));
		
				while (NewLineView.FindChar(TCHAR('\n'), NewLineIndex))
				{
					FStringView _NewLineView = NewLineView.RightChop(NewLineIndex + 1);
					MessageReceived(FString{NewLineView.LeftChop(_NewLineView.Len() + 1)});
					NewLineView = _NewLineView;
				}

				Line = NewLineView;
			}

			return true;
		};

		// wait for tasks to be completed
		while (Status.load(std::memory_order_relaxed) == ELauncherWorkerStatus::Busy)
		{
			FPlatformProcess::Sleep(0.01f);

			ReadLines();

			if (TaskChain.IsValid() && TaskChain->IsChainFinished())
			{
				Status.store(ELauncherWorkerStatus::Completed, std::memory_order_relaxed);

				while (ReadLines());

				// fire off the last line
				if (Line.Len() > 0)
				{
					MessageReceived(MoveTemp(Line));
				}
				break;
			}
		}
	}

	// wait for tasks to be canceled
	if (Status.load(std::memory_order_relaxed) == ELauncherWorkerStatus::Canceling)
	{
		// kill the uat process tree
		if (Process)
		{
			Process.Kill(true);
		}
		
		// kill any lingering target processes left after killing uat
		TerminateLaunchedProcess();

		TaskChain->Cancel();

		while (TaskChain.IsValid() && !TaskChain->IsChainFinished())
		{
			FPlatformProcess::Sleep(0.0);
		}		
	}

	Process = nullptr;
	InputPipe = nullptr;

	if (!TaskChain.IsValid() || Status.load(std::memory_order_relaxed) == ELauncherWorkerStatus::Canceling)
	{
		LaunchCanceled.Broadcast(FPlatformTime::Seconds() - LaunchStartTime);
		Status.store(ELauncherWorkerStatus::Canceled, std::memory_order_relaxed);
	}
	else
	{
		LaunchCompleted.Broadcast(TaskChain->Succeeded(), FPlatformTime::Seconds() - LaunchStartTime, TaskChain->ReturnCode());
	}

	//delete the application@device dictionary
	CachedDevicePackagePair.Empty();

	//stop looking for disconnected devices
	DisableDeviceDiscoveryListener();

	return 0;
}


void FLauncherWorker::Stop( )
{
	Cancel();
}


/* ILauncherWorker overrides
 *****************************************************************************/

void FLauncherWorker::Cancel( )
{
	if (Status.load(std::memory_order_relaxed) == ELauncherWorkerStatus::Busy)
	{
		Status.store(ELauncherWorkerStatus::Canceling, std::memory_order_relaxed);
	}
}


void FLauncherWorker::CancelAndWait( )
{
	if (Status.load(std::memory_order_relaxed) == ELauncherWorkerStatus::Busy)
	{
		Status.store(ELauncherWorkerStatus::Canceling, std::memory_order_relaxed);
		while (Status.load(std::memory_order_relaxed) != ELauncherWorkerStatus::Canceled)
		{
			FPlatformProcess::Sleep(0);
		}
	}
}


int32 FLauncherWorker::GetTasks( TArray<ILauncherTaskPtr>& OutTasks ) const
{
	OutTasks.Reset();

	if (TaskChain.IsValid())
	{
		TQueue<TSharedPtr<FLauncherTask> > Queue;

		Queue.Enqueue(TaskChain);

		TSharedPtr<FLauncherTask> Task;

		// breadth first traversal
		while (Queue.Dequeue(Task))
		{
			OutTasks.Add(Task);

			const TArray<TSharedPtr<FLauncherTask> >& Continuations = Task->GetContinuations();

			for (int32 ContinuationIndex = 0; ContinuationIndex < Continuations.Num(); ++ContinuationIndex)
			{
				Queue.Enqueue(Continuations[ContinuationIndex]);
			}
		}
	}

	return OutTasks.Num();
}


void FLauncherWorker::OnTaskStarted(const FString& TaskName, ILauncherProfileUATCommandPtr UnifiedUATCommand)
{
	StageStartTime = FPlatformTime::Seconds();
	StageStarted.Broadcast(TaskName);
	// look for disconnected devices only after displaying "Running on..."
	if(TaskName.Contains(TEXT("Run Task")))
	{
		EnableDeviceDiscoveryListener(UnifiedUATCommand);
	}
}


void FLauncherWorker::OnTaskCompleted(const FString& TaskName)
{
	StageCompleted.Broadcast(TaskName, FPlatformTime::Seconds() - StageStartTime);
}

static void AddDeviceToLaunchCommand(const FString& DeviceId, TSharedPtr<ITargetDeviceProxy> DeviceProxy, const ILauncherProfileBuildCookRunRef& InBuildCookRun, const ILauncherProfileRef& InProfile, FString& DeviceNames, FString& RoleCommands, bool& bVsyncAdded)
{
	// add the platform
	DeviceNames += TEXT("+\"") + DeviceId + TEXT("\"");
	TArray<ILauncherProfileLaunchRolePtr> Roles;
	if (InBuildCookRun->GetLaunchRolesFor(DeviceId, Roles) > 0)
	{
		for (int32 RoleIndex = 0; RoleIndex < Roles.Num(); RoleIndex++)
		{
			if (!bVsyncAdded && Roles[RoleIndex]->IsVsyncEnabled())
			{
				RoleCommands += TEXT(" -vsync");
				bVsyncAdded = true;
			}
			RoleCommands += *(TEXT(" ") + Roles[RoleIndex]->GetUATCommandLine());
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("nomcp")))
	{
		// if our editor has nomcp then pass it through the launched game
		RoleCommands += TEXT(" -nomcp");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("opengl")))
	{
		RoleCommands += TEXT(" -opengl");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("d3d11")) || FParse::Param(FCommandLine::Get(), TEXT("dx11")))
	{
		RoleCommands += TEXT(" -d3d11");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("d3d12")) || FParse::Param(FCommandLine::Get(), TEXT("dx12")))
	{
		RoleCommands += TEXT(" -d3d12");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("es31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES31")) || FParse::Param(FCommandLine::Get(), TEXT("FeatureLevelES3_1")))
	{
		RoleCommands += TEXT(" -es31");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("sm5")))
	{
		RoleCommands += TEXT(" -sm5");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("sm6")))
	{
		RoleCommands += TEXT(" -sm6");
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("vulkan")))
	{
		FName Variant = DeviceProxy->GetTargetDeviceVariant(DeviceId);
		FString Platform = DeviceProxy->GetTargetPlatformName(Variant);

		bool bCookedVulkan = false;
		bool bCheckTargetedRHIs = false;
		TArray<FString> TargetedShaderFormats;

		if (Platform.StartsWith(TEXT("Windows")))
		{
			FConfigFile WindowsEngineSettings;
			FConfigCacheIni::LoadLocalIniFile(WindowsEngineSettings, TEXT("Engine"), true, TEXT("Windows"));

			bCheckTargetedRHIs = true;
			WindowsEngineSettings.GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("VulkanTargetedShaderFormats"), TargetedShaderFormats);

			TArray<FString> OldConfigShaderFormats;
			WindowsEngineSettings.GetArray(TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings"), TEXT("TargetedRHIs"), OldConfigShaderFormats);

			for (const FString& OldConfigShaderFormat : OldConfigShaderFormats)
			{
				TargetedShaderFormats.AddUnique(OldConfigShaderFormat);
			}
		}
		else if (Platform.StartsWith(TEXT("Linux")))
		{
			FConfigFile LinuxEngineSettings;
			FConfigCacheIni::LoadLocalIniFile(LinuxEngineSettings, TEXT("Engine"), true, TEXT("Linux"));

			bCheckTargetedRHIs = true;
			LinuxEngineSettings.GetArray(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("TargetedRHIs"), TargetedShaderFormats);
		}
		else if (Platform.StartsWith(TEXT("Android")))
		{
			FConfigFile AndroidEngineSettings;
			FConfigCacheIni::LoadLocalIniFile(AndroidEngineSettings, TEXT("Engine"), true, TEXT("Android"));

			bool bAndroidSupportsVulkan, bAndroidSupportsVulkanSM5;
			AndroidEngineSettings.GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bSupportsVulkan"), bAndroidSupportsVulkan);
			AndroidEngineSettings.GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bSupportsVulkanSM5"), bAndroidSupportsVulkanSM5);
			bCookedVulkan = bAndroidSupportsVulkan || bAndroidSupportsVulkanSM5;
			bCheckTargetedRHIs = false;
		}

		if (bCheckTargetedRHIs)
		{
			for (const FString& ShaderFormat : TargetedShaderFormats)
			{
				if (ShaderFormat.StartsWith(TEXT("SF_VULKAN_")))
				{
					bCookedVulkan = true;
				}
			}
		}

		if (bCookedVulkan)
		{
			RoleCommands += TEXT(" -vulkan");
		}
		else
		{
			UE_LOGF(LogLauncherProfile, Warning, "The editor is running on Vulkan, but Vulkan is not enabled for launch platform '%ls'. Launching process with the default RHI.", *Platform);
		}
	}
}

static FString Join(const TSet<FString>& Tokens, const FString& Delimeter)
{
	FString Result;
	for (const FString& Token : Tokens)
	{
		Result+= Delimeter;
		Result+= Token;
	}

	return Result.RightChop(Delimeter.Len());
}

FString FLauncherWorker::CreateUATCommandForBuildCookRun( const ILauncherProfileBuildCookRunRef& InBuildCookRun, const ILauncherProfileRef& InProfile, TArray<FCommandDesc>& OutCommands, FString& CommandStart, const FCommandAdjustments& Adjustments, bool bForTurnkeyCustomBuild )
{
	ILauncherProfileManagerRef LauncherProfileManager = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices").GetProfileManager();

	CommandStart = TEXT("");

	// early out if this will be a no-op
	if (!InBuildCookRun->IsImportingZenSnapshot() &&
		!InBuildCookRun->IsArchiving() &&
		InBuildCookRun->GetBuildMode() == ELauncherProfileBuildModes::DoNotBuild &&
		InBuildCookRun->GetCookMode() == ELauncherProfileCookModes::DoNotCook &&
		InBuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::DoNotDeploy &&
		(InBuildCookRun->GetLaunchMode() == ELauncherProfileLaunchModes::DoNotLaunch || Adjustments.bDisableRun))
	{
		UE_LOGF(LogLauncherProfile, Log, "Skipping BuildCookRun \"%ls\" because it will not do anything", *InBuildCookRun->GetDescription());
		return FString();
	}


	bool bPreferCookedPlatforms = (Adjustments.bPreferCookedPlatforms || InBuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook || InBuildCookRun->ShouldBuild());
	TArray<FString> FinalPlatforms = FinalizePlatforms(InBuildCookRun->GetCookedPlatforms(), InBuildCookRun->GetDeployedDeviceGroup(), bPreferCookedPlatforms);

	bool bRun = (InBuildCookRun->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch && !Adjustments.bDisableRun);

	// base UAT command arguments
	static const FString ConfigStrings[] = { TEXT("Unknown"), TEXT("Debug"), TEXT("DebugGame"), TEXT("Development"), TEXT("Shipping"), TEXT("Test") };
	FString UATCommand = FString::Printf(TEXT("BuildCookRun -project=\"%s\" -clientconfig=%s -serverconfig=%s"), // @todo: don't have to always add -clientconfig & -serverconfig
		*FPaths::ConvertRelativePathToFull(InProfile->GetProjectPath()),
		LexToString(InBuildCookRun->GetBuildConfiguration()),
		LexToString(InBuildCookRun->GetBuildConfiguration()));

	// we never want to build the editor when launching from the editor or running with an installed engine (which can't rebuild itself)
	UATCommand += GIsEditor || FApp::IsEngineInstalled() ? TEXT(" -nocompileeditor") : TEXT("");
	UATCommand += FApp::IsEngineInstalled() ? TEXT(" -installed") : TEXT("");

	// specify the path to the editor exe if necessary
	FString EditorExe =  InProfile->GetEditorExe();
	if(EditorExe.Len() > 0)
	{
		UATCommand += FString::Printf(TEXT(" -unrealexe=\"%s\""), *EditorExe);
	}

	FGuid SessionId(FGuid::NewGuid());
	FString InitialMap = InBuildCookRun->GetDefaultLaunchRole()->GetInitialMap();
	if (InitialMap.IsEmpty() && InBuildCookRun->GetCookedMaps().Num() == 1)
	{
		InitialMap = InBuildCookRun->GetCookedMaps()[0];
	}

	// staging directory
	FString StageDirectory = TEXT("");
	auto PackageDirectory = InBuildCookRun->GetPackageDirectory();
	if (PackageDirectory.Len() > 0)
	{
		StageDirectory += FString::Printf(TEXT(" -stagingdirectory=\"%s\""), *PackageDirectory);
	}

	// determine if there is a server platform
	FString ServerCommand = TEXT("");
	FString ServerPlatforms = TEXT("");
	FString Platforms = TEXT("");
	FString PlatformCommand = TEXT("");
	FString OptionalParams = TEXT("");
	TSet<FString> OptionalTargetPlatforms;
	TSet<FString> OptionalCookFlavors;
	TSet<EBuildTargetType> BuildTargets;
	
	bool bUATClosesAfterLaunch = false;
	for (int32 PlatformIndex = 0; PlatformIndex < FinalPlatforms.Num(); ++PlatformIndex)
	{
		// Platform info for the given platform
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(*FinalPlatforms[PlatformIndex]));
		if (bForTurnkeyCustomBuild && PlatformInfo == nullptr)
		{
			continue;
		}

		if (ensure(PlatformInfo))
		{
			// separate out Server platforms
			FString& PlatformString = (PlatformInfo->PlatformType == EBuildTargetType::Server) ? ServerPlatforms : Platforms;

			PlatformString += TEXT("+");
			PlatformString += PlatformInfo->DataDrivenPlatformInfo->UBTPlatformString;

			// Append any extra UAT flags specified for this platform flavor
			if (!PlatformInfo->UATCommandLine.IsEmpty())
			{
				FString OptionalUATCommandLine = PlatformInfo->UATCommandLine;

				FString OptionalTargetPlatform;
				if (FParse::Value(*OptionalUATCommandLine, TEXT("-targetplatform="), OptionalTargetPlatform))
				{
					OptionalTargetPlatforms.Add(OptionalTargetPlatform);
					OptionalUATCommandLine.ReplaceInline(*(TEXT("-targetplatform=") + OptionalTargetPlatform), TEXT(""));
				}

				FString OptionalCookFlavor;
				if (FParse::Value(*OptionalUATCommandLine, TEXT("-cookflavor="), OptionalCookFlavor))
				{
					OptionalCookFlavors.Add(OptionalCookFlavor);
					OptionalUATCommandLine.ReplaceInline(*(TEXT("-cookflavor=") + OptionalCookFlavor), TEXT(""));
				}

				OptionalParams += TEXT(" ");
				OptionalParams += OptionalUATCommandLine;
			}
			bUATClosesAfterLaunch |= PlatformInfo->DataDrivenPlatformInfo->bUATClosesAfterLaunch;

			BuildTargets.Add(PlatformInfo->PlatformType);
		}
	}

	// If both Client/Game and Server are desired to be built avoid Server causing clients/game to not be built PlatformInfo wise
	if (ServerPlatforms.Len() > 0 && Platforms.Len() > 0 && OptionalParams.Contains(TEXT("-noclient")))
	{
		OptionalParams = OptionalParams.Replace(TEXT("-noclient"), TEXT(""));
	}

	if (ServerPlatforms.Len() > 0)
	{
		ServerCommand = TEXT(" -server -serverplatform=") + ServerPlatforms.RightChop(1);
		if (Platforms.Len() == 0)
		{
			OptionalParams += TEXT(" -noclient");
		}

		if (InBuildCookRun->GetServerArchitectures().Num() > 0)
		{
			ServerCommand += TEXT(" -serverarchitecture=") + FString::Join(InBuildCookRun->GetServerArchitectures(), TEXT("+") );
		}
	}
	bool bSetClientArchitecture = false;
	if (Platforms.Len() > 0)
	{
		PlatformCommand = TEXT(" -platform=") + Platforms.RightChop(1);
		bSetClientArchitecture = true;
	}
	
	UATCommand += PlatformCommand;
	UATCommand += ServerCommand;
	UATCommand += OptionalParams;

	if (OptionalTargetPlatforms.Num() > 0)
	{
		UATCommand += (TEXT(" -targetplatform=") + Join(OptionalTargetPlatforms, TEXT("+")));
	}
	
	if (OptionalCookFlavors.Num() > 0)
	{
		UATCommand += (TEXT(" -cookflavor=") + Join(OptionalCookFlavors, TEXT("+")));
	}

	if (InBuildCookRun->GetBuildTargets().Num() > 0 && !InBuildCookRun->GetBuildTargets().Contains(FString()))
	{
		UATCommand += TEXT(" -target=") + FString::Join(InBuildCookRun->GetBuildTargets(), TEXT("+"));
	}

	if (InBuildCookRun->IsDeviceASimulator())
	{
		if (Platforms.Contains(TEXT("IOS")))
		{
			UATCommand += TEXT(" -clientarchitecture=iossimulator");
			bSetClientArchitecture = false;
		}
		// TODO: add tvOS and VisionOS simulators below
	}

	if (bSetClientArchitecture && InBuildCookRun->GetClientArchitectures().Num() > 0)
	{
		UATCommand += TEXT(" -clientarchitecture=") + FString::Join(InBuildCookRun->GetClientArchitectures(), TEXT("+") );
	}


	// device list
	FString DeviceNames = TEXT("");
	FString DeviceCommand = TEXT("");
	FString RoleCommands = TEXT("");
	ILauncherDeviceGroupPtr DeviceGroup = InBuildCookRun->GetDeployedDeviceGroup();

	bool bVsyncAdded = false;

	if (DeviceGroup.IsValid() && DeviceProxyManager.IsValid())
	{
		const TArray<FString>& Devices = DeviceGroup->GetDeviceIDs();

		if (Devices.Num() > 0)
		{
			// for each deployed device...
			for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
			{
				const FString& DeviceId = Devices[DeviceIndex];
				TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceId);
				if (DeviceProxy.IsValid())
				{
					AddDeviceToLaunchCommand(DeviceId, DeviceProxy, InBuildCookRun, InProfile, DeviceNames, RoleCommands, bVsyncAdded);

					// also add the credentials, if necessary
					FString DeviceUser = DeviceProxy->GetDeviceUser();
					if (DeviceUser.Len() > 0)
					{
						DeviceCommand += FString::Printf(TEXT(" -deviceuser=%s"), *DeviceUser);
					}

					FString DeviceUserPassword = DeviceProxy->GetDeviceUserPassword();
					if (DeviceUserPassword.Len() > 0)
					{
						DeviceCommand += FString::Printf(TEXT(" -devicepass=%s"), *DeviceUserPassword);
					}
				}
			}
		}
		else
		{
			RoleCommands = InBuildCookRun->GetDefaultLaunchRole()->GetUATCommandLine();
		}
	}

	if (DeviceNames.Len() > 0)
	{
		DeviceCommand += TEXT(" -device=") + DeviceNames.RightChop(1);
	}

	// game command line
	//@fixme: this is destined for UECommandLine.txt ... probably want initial map (if set) but not -Messaging ?
	FString CommandLine = FString::Printf(TEXT(" -cmdline=\"%s%s\""),
		*InitialMap, 
		bForTurnkeyCustomBuild ? TEXT("") : TEXT(" -Messaging") );
	if (CommandLine.EndsWith(TEXT("\"\"")))
	{
		CommandLine.Reset(); // don't bother with -cmdline if it's empty
	}

	// localization command line
	FString LocalizationCommands;
#if WITH_EDITOR
	const FString PreviewGameLanguage = FTextLocalizationManager::Get().GetConfiguredGameLocalizationPreviewLanguage();
	if (!PreviewGameLanguage.IsEmpty())
	{
		LocalizationCommands += TEXT(" -culture=");
		LocalizationCommands += PreviewGameLanguage;
	}
#endif	// WITH_EDITOR

	const bool bOnlyInherited = true; // Only getting inherited, not explicitly set subprocess commandline args due to issues when passing
										// the -Multiprocess explicitly set commandline argument which prevents configs from saving/writing
	if (bRun)
	{
		// to reduce UECommandLine.txt churn (timestamp causing extra work), for LaunchOn (ie iterative deploy) we use a single session guid
		if (InBuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyToDevice && InBuildCookRun->IsDeployingIncrementally())
		{
			static FGuid StaticGuid(FGuid::NewGuid());
			SessionId = StaticGuid;
		}

		// additional commands to be sent to the commandline
		FString SessionCommands;
		if (!bForTurnkeyCustomBuild)
		{
			FString SessionName = InProfile->GetName().Replace(TEXT("\'"), TEXT("_")).Replace(TEXT("\'"), TEXT("_")).Replace(TEXT("&"), TEXT("_"));
			FString SessionOwner = FString(FPlatformProcess::UserName(false)).Replace(TEXT("\'"), TEXT("_")).Replace(TEXT("\'"), TEXT("_"));;
			SessionCommands = FString::Printf(TEXT("-SessionId=%s -SessionOwner='%s' -SessionName='%s' "),
			*SessionId.ToString(),
			*SessionOwner,
			*SessionName);
		}

		// allow external items to adjust the command line
		FString ProfileAdditionalCommandLineParameters = InBuildCookRun->GetAdditionalCommandLineParameters();
		LauncherProfileManager->OnPostProcessLaunchCommandLine().Broadcast(InProfile, EBuildTargetType::Unknown, ProfileAdditionalCommandLineParameters);
		// Escape embedded double quotes so they survive the outer -addcmdline="..." wrapper applied below.
		ProfileAdditionalCommandLineParameters.ReplaceInline(TEXT("\""), TEXT("\\\""));

		FString AdditionalCommandLine;
		TStringBuilder<64> InheritableGameOptions;
		ECommandLineArgumentFlags CommandLineContextFlags = ECommandLineArgumentFlags::ClientContext | ECommandLineArgumentFlags::ServerContext;
		FCommandLine::BuildSubprocessCommandLine(CommandLineContextFlags, bOnlyInherited, InheritableGameOptions);
		FString EscapedInheritableGameOptions(InheritableGameOptions);
		EscapedInheritableGameOptions.ReplaceInline(TEXT("\""), TEXT("\\\""));
		FString AddCmdLine = FString::Printf(TEXT("%s%s %s %s%s"),
			*SessionCommands,
			*RoleCommands,
			*LocalizationCommands,
			*ProfileAdditionalCommandLineParameters,
			*EscapedInheritableGameOptions);
		if (!AddCmdLine.TrimStartAndEnd().IsEmpty()) // don't bother with -addcmdline if it's empty
		{
			AdditionalCommandLine = FString::Printf(TEXT(" -addcmdline=\"%s\""), *FinalizeCommandLine(AddCmdLine));
		}

		for (EBuildTargetType BuildTarget : BuildTargets)
		{
			FString BuildTargetCmdLine = InBuildCookRun->GetAdditionalTargetCommandLineParameters(BuildTarget);
			LauncherProfileManager->OnPostProcessLaunchCommandLine().Broadcast(InProfile, BuildTarget, BuildTargetCmdLine);
			if (BuildTargetCmdLine.TrimStartAndEnd().IsEmpty())
			{
				continue;		
			}
			BuildTargetCmdLine.ReplaceInline(TEXT("\""), TEXT("\\\""));

			switch (BuildTarget)
			{
				case EBuildTargetType::Server:
				{
					AdditionalCommandLine += FString::Printf(TEXT(" -servercmdline=\"%s\""), *FinalizeCommandLine(BuildTargetCmdLine));
				}
				break;

				case EBuildTargetType::Client:
				case EBuildTargetType::Game:
				{
					AdditionalCommandLine += FString::Printf(TEXT(" -clientcmdline=\"%s\""), *FinalizeCommandLine(BuildTargetCmdLine));
				}
				break;
			}
		}

		UATCommand += AdditionalCommandLine;
	}

	bool bPreStaged = InBuildCookRun->IsUsingPreStagedBuild();


	// map list
	FString MapList;
	const TArray<FString>& CookedMaps = InBuildCookRun->GetCookedMaps();
	if (CookedMaps.Num() > 0 && (InBuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook || InBuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBookInEditor))
	{
		if (!InitialMap.IsEmpty())
		{
			MapList += InitialMap;
			MapList += TEXT("+");
		}
		MapList += FString::Join(CookedMaps, TEXT("+"));
	}
	else
	{
		MapList = InitialMap;
	}
	if (!MapList.IsEmpty())
	{
		MapList = FString(TEXT(" -map=")) + MapList;
	}

	// culture list
	FString CultureList;
	{
		const TArray<FString>& CookedCultures = InBuildCookRun->GetCookedCultures();
		if (CookedCultures.Num() > 0 && (InBuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook || InBuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBookInEditor))
		{
			CultureList += TEXT(" -CookCultures=");
			CultureList += FString::Join(CookedCultures, TEXT("+"));
		}
	}

	bool bIsBuilding = InBuildCookRun->ShouldBuild();

	// build
	if (bIsBuilding && !bPreStaged)
	{
		UATCommand += TEXT(" -build");

		FCommandDesc Desc;
		FText Command = FText::Format(LOCTEXT("LauncherBuildDesc", "Build game for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
		Desc.Name = "Build Task";
		Desc.Desc = Command.ToString();
		Desc.EndText = TEXT("********** BUILD COMMAND COMPLETED **********");
		OutCommands.Add(Desc);
		CommandStart = TEXT("********** BUILD COMMAND STARTED **********");
		// @todo: server
	}
	
	if (InBuildCookRun->IsFastIterate())
	{
		UATCommand += TEXT(" -FastIterate -ubtargs=\"-FastIterate\"");
	}

	// snapshot import
	if (InBuildCookRun->IsImportingZenSnapshot() && !bPreStaged)
	{
		UATCommand += TEXT(" -snapshot");

		FCommandDesc Desc;
		FText Command = FText::Format(LOCTEXT("LauncherSnapshotDesc", "Import Zen snapshot for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
		Desc.Name = "Snapshot Import Task";
		Desc.Desc = Command.ToString();
		Desc.EndText = TEXT("********** SNAPSHOT IMPORT COMPLETED **********");
		OutCommands.Add(Desc);
		CommandStart = TEXT("********** SNAPSHOT IMPORT STARTED **********");
	}

	// cook
	if (!bPreStaged)
	{
		switch(InBuildCookRun->GetCookMode())
		{
		case ELauncherProfileCookModes::ByTheBook:
			{
				UATCommand += TEXT(" -cook");

				UATCommand += MapList;
				UATCommand += CultureList;

				if (InBuildCookRun->IsCookingUnversioned())
				{
					UATCommand += TEXT(" -unversionedcookedcontent");
				}

				if (InBuildCookRun->IsEncryptingIniFiles())
				{
					UATCommand += TEXT(" -encryptinifiles");
				}

				TStringBuilder<64> InheritableCookOptions;
				FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags::CommandletContext, bOnlyInherited, InheritableCookOptions);
				FString AdditionalOptions = InBuildCookRun->GetCookOptions();
				if (!AdditionalOptions.IsEmpty() || (InheritableCookOptions.Len() > 0))
				{
					UATCommand += TEXT(" -additionalcookeroptions=\"");

					// Escape any quotes in the argument list
					UATCommand += AdditionalOptions.Replace(TEXT("\""), TEXT("\\\""));

					if (InheritableCookOptions.Len() > 0)
					{
						if (!AdditionalOptions.IsEmpty())
						{
							UATCommand += TEXT(" ");
						}
						FString EscapedInheritableCookOptions(InheritableCookOptions);
						EscapedInheritableCookOptions.ReplaceInline(TEXT("\""), TEXT("\\\""));
						UATCommand += EscapedInheritableCookOptions;
					}

					// If the additional options ends with a slash, make sure we don't escape the quote
					if (UATCommand.EndsWith("\\"))
					{
						UATCommand += TEXT("\\");
					}

					UATCommand += TEXT("\"");
				}

				if (FParse::Param(FCommandLine::Get(), TEXT("fastcook")))
				{
					// if our editor has nomcp then pass it through the launched game
					UATCommand += TEXT(" -fastcook");
				}

				if (InBuildCookRun->IsUsingZenStore())
				{
					UATCommand += TEXT(" -zenstore");
				}

				if (FDerivedDataCacheInterface* DDC = TryGetDerivedDataCache())
				{
					const TCHAR* GraphName = DDC->GetGraphName();
					if (FCString::Strcmp(GraphName, DDC->GetDefaultGraphName()))
					{
						UATCommand += FString::Printf(TEXT(" -DDC=%s"), GraphName);
					}
				}

				if (InBuildCookRun->IsPackingWithUnrealPak())
				{
					UATCommand += TEXT(" -pak");
					UATCommand += TEXT(" -stage");
					if (InBuildCookRun->IsUsingIoStore())
					{
						UATCommand += TEXT(" -iostore");
					}
					if (InBuildCookRun->IsCompressed())
					{
						UATCommand += TEXT(" -compressed");
					}
				}

				if (InBuildCookRun->MakeBinaryConfig())
				{
					UATCommand += TEXT(" -makebinaryconfig");
				}

				if ( InBuildCookRun->IsCreatingReleaseVersion() )
				{
					UATCommand += TEXT(" -createreleaseversion=");
					UATCommand += InBuildCookRun->GetCreateReleaseVersionName();
				
				}

				if ( InBuildCookRun->IsCreatingDLC() )
				{
					UATCommand += TEXT(" -dlcname=");
					UATCommand += InBuildCookRun->GetDLCName();
				}

				if ( InBuildCookRun->IsDLCIncludingEngineContent() )
				{
					UATCommand += TEXT(" -DLCIncludeEngineContent");
				}

				if ( InBuildCookRun->IsGeneratingPatch() )
				{
					UATCommand += TEXT(" -generatepatch");

					if ( InBuildCookRun->ShouldAddPatchLevel() )
					{
						UATCommand += TEXT(" -addpatchlevel");
					}
				}

				if ( InBuildCookRun->IsGeneratingPatch() || 
					InBuildCookRun->IsCreatingReleaseVersion() || 
					InBuildCookRun->IsCreatingDLC() )
				{
					if ( InBuildCookRun->GetBasedOnReleaseVersionName().IsEmpty() == false )
					{
						UATCommand += TEXT(" -basedonreleaseversion=");
						UATCommand += InBuildCookRun->GetBasedOnReleaseVersionName();
					}

					if (InBuildCookRun->GetOriginalReleaseVersionName().IsEmpty() == false)
					{
						UATCommand += TEXT(" -originalreleaseversion=");
						UATCommand += InBuildCookRun->GetOriginalReleaseVersionName();
					}
				}

				if (InBuildCookRun->IsGeneratingChunks())
				{
					UATCommand += TEXT(" -manifests");
				}

				if (InBuildCookRun->IsGenerateHttpChunkData())
				{
					auto Cmd = FString::Printf(TEXT(" -createchunkinstall -chunkinstalldirectory=\"%s\" -chunkinstallversion=\"%s\""), *InBuildCookRun->GetHttpChunkDataDirectory(), *InBuildCookRun->GetHttpChunkDataReleaseName());
					UATCommand += Cmd;
				}
			
				// Creating a packed DLC requires staging
				if (InBuildCookRun->GetPackagingMode() == ELauncherProfilePackagingModes::DoNotPackage && InBuildCookRun->IsCreatingDLC() && InBuildCookRun->IsPackingWithUnrealPak())
				{
					UATCommand += TEXT(" -stage");
				}

				FCommandDesc Desc;
				FText Command = FText::Format(LOCTEXT("LauncherCookDesc", "Cook content for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
				Desc.Name = "Cook Task";
				Desc.Desc = Command.ToString();
				Desc.EndText = TEXT("********** COOK COMMAND COMPLETED **********");
				OutCommands.Add(Desc);
				if (CommandStart.Len() == 0)
				{
					CommandStart = TEXT("********** COOK COMMAND STARTED **********");
				}
			}
			break;
		case ELauncherProfileCookModes::OnTheFly:
			{
				UATCommand += TEXT(" -cookonthefly");
			
				if (InBuildCookRun->IsUsingZenStore())
				{
					UATCommand += TEXT(" -zenstore");
				}

				TStringBuilder<64> InheritableCookOptions;
				FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags::CommandletContext, bOnlyInherited, InheritableCookOptions);

				FString AdditionalOptions = InBuildCookRun->GetCookOptions();
				if (!AdditionalOptions.IsEmpty() || (InheritableCookOptions.Len() > 0))
				{
					UATCommand += TEXT(" -additionalcookeroptions=\"");

					// Escape any quotes in the argument list
					UATCommand += AdditionalOptions.Replace(TEXT("\""), TEXT("\\\""));

					if (InheritableCookOptions.Len() > 0)
					{
						if (!AdditionalOptions.IsEmpty())
						{
							UATCommand += TEXT(" ");
						}
						FString EscapedInheritableCookOptions(InheritableCookOptions);
						EscapedInheritableCookOptions.ReplaceInline(TEXT("\""), TEXT("\\\""));
						UATCommand += EscapedInheritableCookOptions;
					}

					// If the additional options ends with a slash, make sure we don't escape the quote
					if (UATCommand.EndsWith("\\"))
					{
						UATCommand += TEXT("\\");
					}

					UATCommand += TEXT("\"");
				}

				if (FDerivedDataCacheInterface* DDC = GetDerivedDataCache())
				{
					UATCommand += FString::Printf(TEXT(" -ddc=%s"), DDC->GetGraphName());
				}

				//if UAT doesn't stick around as long as the process we are going to run, then we can't kill the COTF server when UAT goes down because the program
				//will still need it.  If UAT DOES stick around with the process then we DO want the COTF server to die with UAT so the next time we launch we don't end up
				//with two COTF servers.
				if (bUATClosesAfterLaunch)
				{
					UATCommand += " -nokill";
				}
				UATCommand += MapList;

				FCommandDesc Desc;
				FText Command = LOCTEXT("LauncherCookOnTheFlyDesc", "Starting cook on the fly server");
				Desc.Name = "Cook Server Task";
				Desc.Desc = Command.ToString();
				Desc.EndText = TEXT("********** COOK COMMAND COMPLETED **********");
				OutCommands.Add(Desc);
				if (CommandStart.Len() == 0)
				{
					CommandStart = TEXT("********** COOK COMMAND STARTED **********");
				}
			}
			break;
		case ELauncherProfileCookModes::OnTheFlyInEditor:
			UATCommand += MapList;
			UATCommand += " -skipcook -cookonthefly -CookInEditor";
			if (InBuildCookRun->IsUsingZenStore())
			{
				UATCommand += TEXT(" -zenstore");
			}
			break;
		case ELauncherProfileCookModes::ByTheBookInEditor:
			UATCommand += MapList;
			UATCommand += CultureList;
			UATCommand += TEXT(" -skipcook -CookInEditor"); // don't cook anything the editor is doing it ;)
			if (InBuildCookRun->IsUsingZenStore())
			{
				UATCommand += TEXT(" -zenstore");
			}
			if (InBuildCookRun->IsPackingWithUnrealPak())
			{
				UATCommand += TEXT(" -pak");
				if (InBuildCookRun->IsUsingIoStore())
				{
					UATCommand += TEXT(" -iostore");
				}
				if (InBuildCookRun->IsCompressed())
				{
					UATCommand += TEXT(" -compressed");
				}
			}
			break;
		case ELauncherProfileCookModes::DoNotCook:
			UATCommand += TEXT(" -skipcook");
			if (InBuildCookRun->IsUsingZenPakStreaming())
			{
				UATCommand += FString::Printf(TEXT(" -skippak -ZenWorkspaceSharePath=\\\"%s\\\" "), *InBuildCookRun->GetZenPakStreamingPath() );
			}
			else if (InBuildCookRun->IsImportingZenSnapshot() && InBuildCookRun->IsPackingWithUnrealPak())
			{
				UATCommand += TEXT(" -pak");
				if (InBuildCookRun->IsUsingIoStore())
				{
					UATCommand += TEXT(" -iostore");
				}
				if (InBuildCookRun->IsCompressed())
				{
					UATCommand += TEXT(" -compressed");
				}
			}
			break;
		}

		if ( InBuildCookRun->IsForDistribution() )
		{
			UATCommand += TEXT(" -distribution");
		}

		if (InBuildCookRun->IsCookingIncrementally())
		{
			UATCommand += TEXT(" -iterativecooking");
		}

		if ( InBuildCookRun->IsIterateSharedCookedBuild() )
		{
			UATCommand += TEXT(" -iteratesharedcookedbuild=usesyncedbuild");
		}

		if (InBuildCookRun->GetSkipCookingEditorContent())
		{
			UATCommand += TEXT(" -SkipCookingEditorContent");
		}
	}

	FString StageAdditionalCommandLine;
	if (InBuildCookRun->IsUsingIoStore() &&
		InBuildCookRun->GetReferenceContainerGlobalFileName().Len())
	{
		StageAdditionalCommandLine += TEXT(" -ReferenceContainerGlobalFileName=\"") + InBuildCookRun->GetReferenceContainerGlobalFileName() + TEXT("\"");
		if (InBuildCookRun->GetReferenceContainerCryptoKeysFileName().Len())
		{
			StageAdditionalCommandLine += TEXT(" -ReferenceContainerCryptoKeys=\"") + InBuildCookRun->GetReferenceContainerCryptoKeysFileName() + TEXT("\"") ;
		}
	}

	// stage/package/deploy
	if (bPreStaged)
	{
		UATCommand += TEXT(" -prestaged");
	}

	if (InBuildCookRun->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy)
	{
		switch (InBuildCookRun->GetDeploymentMode())
		{
		case ELauncherProfileDeploymentModes::CopyRepository:
			{
				UATCommand += TEXT(" -skipstage -deploy");
				UATCommand += CommandLine;
				UATCommand += StageDirectory;
				UATCommand += DeviceCommand;

				FCommandDesc Desc;
				FText Command = FText::Format(LOCTEXT("LauncherDeployDesc", "Deploying content for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
				Desc.Name = "Deploy Task";
				Desc.Desc = Command.ToString();
				Desc.EndText = TEXT("********** DEPLOY COMMAND COMPLETED **********");
				OutCommands.Add(Desc);
				if (CommandStart.Len() == 0)
				{
					CommandStart = TEXT("********** DEPLOY COMMAND STARTED **********");
				}
			}
			break;

		case ELauncherProfileDeploymentModes::CopyToDevice:
			{
				if (InBuildCookRun->IsDeployingIncrementally())
				{
					UATCommand += " -iterativedeploy";
				}
				if (InBuildCookRun->GetPackagingMode() == ELauncherProfilePackagingModes::Locally)
				{
					UATCommand += TEXT(" -package");
				}
			}
		case ELauncherProfileDeploymentModes::FileServer:
			{
				UATCommand += TEXT(" -stage -deploy");
				UATCommand += CommandLine;
				UATCommand += StageDirectory;
				UATCommand += DeviceCommand;
				UATCommand += StageAdditionalCommandLine;

				FCommandDesc Desc;
				FText Command = FText::Format(LOCTEXT("LauncherDeployDesc", "Deploying content for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
				Desc.Name = "Deploy Task";
				Desc.Desc = Command.ToString();
				Desc.EndText = TEXT("********** DEPLOY COMMAND COMPLETED **********");
				OutCommands.Add(Desc);
				if (CommandStart.Len() == 0)
				{
					CommandStart = TEXT("********** STAGE COMMAND STARTED **********");
				}
			}
			break;
		}
	}
	else
	{
		if (InBuildCookRun->IsIncludingPrerequisites())
		{
			UATCommand += TEXT(" -prereqs");
		}

		if (InBuildCookRun->GetPackagingMode() == ELauncherProfilePackagingModes::Locally)
		{
			UATCommand += TEXT(" -stage -package");
			UATCommand += StageDirectory;
			UATCommand += CommandLine;
			UATCommand += StageAdditionalCommandLine;

			FCommandDesc Desc;
			FText Command = FText::Format(LOCTEXT("LauncherPackageDesc", "Packaging content for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
			Desc.Name = "Package Task";
			Desc.Desc = Command.ToString();
			Desc.EndText = TEXT("********** PACKAGE COMMAND COMPLETED **********");
			OutCommands.Add(Desc);
			if (CommandStart.Len() == 0)
			{
				CommandStart = TEXT("********** STAGE COMMAND STARTED **********");
			}
		}

		if (InBuildCookRun->IsArchiving())
		{
			UATCommand += FString::Printf(TEXT(" -archive -archivedirectory=\"%s\""), *InBuildCookRun->GetArchiveDirectory());

			FCommandDesc Desc;
			FText Command = FText::Format(LOCTEXT("LauncherArchiveDesc", "Archiving content for {0}"), FText::FromString(FString::Join(FinalPlatforms, TEXT("+") )));
			Desc.Name = "Archive Task";
			Desc.Desc = Command.ToString();
			Desc.EndText = TEXT("********** ARCHIVE COMMAND COMPLETED **********");
			OutCommands.Add(Desc);
			if (CommandStart.Len() == 0)
			{
				CommandStart = TEXT("********** ARCHIVE COMMAND STARTED **********");
			}
		}
	}

	// run
	if (bRun)
	{
		UATCommand += TEXT(" -run ");

		FCommandDesc Desc;
		FText Command = FText::Format(LOCTEXT("LauncherRunDesc", "Launching on {0}"), FText::FromString(DeviceNames.RightChop(1)));
		Desc.Name = "Run Task";
		Desc.Desc = Command.ToString();
		Desc.EndText = TEXT("********** RUN COMMAND COMPLETED **********");
		OutCommands.Add(Desc);
		if (CommandStart.Len() == 0)
		{
			CommandStart = TEXT("********** RUN COMMAND STARTED **********");
		}
	}


	FString AdditionalUATCommandLine = InBuildCookRun->GetAdditionalUATCommandLine();
	if (!AdditionalUATCommandLine.IsEmpty())
	{
		AdditionalUATCommandLine.ReplaceInline(TEXT("\""), TEXT("\\\""));

		UATCommand += TEXT(" ");
		UATCommand += AdditionalUATCommandLine;
	}

	// Dispatch the event which may further customize the command line passed to the automated test.
	FUATCommandPostProcessParameters PostProcessParameters =
	{
		.bShouldBeSkipped = false, // don't skip by default
		.Profile = InProfile,
		.UATCommand = InBuildCookRun,
		.UATCommandLine = UATCommand,
		.Commands = OutCommands,
		.CommandStart = CommandStart,
	};
	LauncherProfileManager->OnPostProcessUATCommandLine().Broadcast(PostProcessParameters);
	if (PostProcessParameters.bShouldBeSkipped)
	{
		UE_LOGF(LogLauncherProfile, Log, "skipping BuildCookRun %ls", InBuildCookRun->GetInternalName() );
		return FString();
	}



	return FinalizeCommandLine(UATCommand);
}


FString FLauncherWorker::CreateUATCommandForAutomatedTest( const ILauncherProfileAutomatedTestRef& InAutomatedTest, const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms, TArray<FCommandDesc>& OutCommands, FString& OutCommandStart, const FCommandAdjustments& Adjustments )
{
	checkf(InAutomatedTest->GetOrder() >= 0, TEXT("Automated tests should not have negative order"));


	// skip disabled tests
	if (!InAutomatedTest->IsEnabled())
	{
		return FString();
	}
	
	ILauncherProfileManagerRef LauncherProfileManager = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices").GetProfileManager();


	// @todo: could potentially add BuildConfiguation, LocalExecutable, Command Line, DeviceIDs and Platforms to each automated test so they can be run without a BCR... but maybe an overkill?
	// ...for now, generate values based on unified values from the BuildCookRuns
	EBuildConfiguration BuildConfiguration = EBuildConfiguration::Development;
	bool bUsingLocalExecutables = false;
	FString LaunchCommandLineParameters;
	TArray<FString> DeviceIDs;
	TArray<FString> SelectedPlatforms = InPlatforms;
	for (ILauncherProfileBuildCookRunRef BuildCookRun : InProfile->GetBuildCookRunCommands())
	{
		if (BuildCookRun->IsEnabled())
		{
			BuildConfiguration = FMath::Max( BuildCookRun->GetBuildConfiguration(), BuildConfiguration );
			LaunchCommandLineParameters += TEXT(" ");
			LaunchCommandLineParameters += BuildCookRun->GetAdditionalCommandLineParameters();
			bUsingLocalExecutables |= BuildCookRun->ShouldBuild();
			if (BuildCookRun->GetDeployedDeviceGroup().IsValid())
			{
				for (const FString& DeviceID : BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs())
				{
					DeviceIDs.AddUnique(DeviceID);
				}
			}
		}
	}
	LaunchCommandLineParameters.TrimStartInline();


	// basic command parameters
	FString AutomatedTestCommand = FString::Printf( TEXT("%s -utf8output -nop4 "), *InAutomatedTest->GetUATCommand() );
	AutomatedTestCommand += FString::Printf(TEXT("-project=\"%s\" "), *FPaths::ConvertRelativePathToFull(InProfile->GetProjectPath()));
	AutomatedTestCommand += FString::Printf(TEXT("-configuration=%s "), LexToString(BuildConfiguration) );
	AutomatedTestCommand += FString::Printf(TEXT("-tests=%s "), *InAutomatedTest->GetTests());

	// collect platforms
	FString ServerPlatforms = TEXT("");
	FString Platforms = TEXT("");
	for (int32 PlatformIndex = 0; PlatformIndex < SelectedPlatforms.Num(); ++PlatformIndex)
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(*SelectedPlatforms[PlatformIndex]));
		if (ensure(PlatformInfo))
		{
			FString& PlatformString = (PlatformInfo->PlatformType == EBuildTargetType::Server) ? ServerPlatforms : Platforms;
			PlatformString += TEXT("+");
			PlatformString += PlatformInfo->DataDrivenPlatformInfo->UBTPlatformString;
		}
	}
	if (ServerPlatforms.Len() > 0)
	{
		AutomatedTestCommand += FString::Printf(TEXT("-serverplatform=%s "), *ServerPlatforms.RightChop(1) );
	}
	if (Platforms.Len() > 0)
	{
		AutomatedTestCommand += FString::Printf(TEXT("-platform=%s "), *Platforms.RightChop(1) );
	}

	// specify the build to use
	if (InProfile->GetAutomatedTestBuildPath().IsEmpty() || !InProfile->IsUsingAutomatedTestBuild())
	{
		AutomatedTestCommand += TEXT("-build=local ");
	}
	else
	{
		AutomatedTestCommand += FString::Printf( TEXT("-build=\"%s\" "), *InProfile->GetAutomatedTestBuildPath());
	}

	// use the local executables we just made if necessary
	if (bUsingLocalExecutables)
	{
		AutomatedTestCommand += TEXT("-dev ");
	}

	// Device ID passed down from Project Launcher can contain
	// "Windows" instead of "Win64" in the Device Id, which is 
	// not recognized by "RunUnreal" UAT command. In this case,
	// we convert the Device Id as required. 
	const auto ConvertDeviceID = [](FString DeviceID) -> FString
	{
		if (DeviceID.Contains("Windows"))
		{
			DeviceID = DeviceID.Replace(TEXT("Windows"), TEXT("Win64"));
		}

		return DeviceID.TrimStartAndEnd();
	};

	// specify the devices to use
	if (DeviceIDs.Num() > 0)
	{
		FString DeviceNames;
		for (const FString& DeviceId : DeviceIDs)
		{
			FString FinalDeviceId = DeviceId.TrimStartAndEnd().Replace(TEXT("@"), TEXT(":"));
			if (!FinalDeviceId.IsEmpty())
			{
				DeviceNames += TEXT(",\"") + ConvertDeviceID(FinalDeviceId) + TEXT("\"");
			}
		}
		if (!DeviceNames.IsEmpty())
		{
			AutomatedTestCommand += FString::Printf(TEXT("-devices=%s "), *DeviceNames.RightChop(1));
		}
	}

	// Profile 'additional command line parameters' are commands for the app wrapped in the flag -addcmdline in CreateUATCommand. Wrap the value if 
	// it exists in the automated test equivalent which is -Args. 
	FString AdditionalCommandLine;

	// Append the tests command line parameters
	FString LaunchAdditionalCommandLine = LaunchCommandLineParameters;
	if (!LaunchAdditionalCommandLine.IsEmpty())
	{
		AdditionalCommandLine = FString::Printf(TEXT("-Args=\"%s\""), *LaunchAdditionalCommandLine.Replace(TEXT("\""), TEXT("\\\"")));
	}

	// Append the tests additional command line parameters
	FString AdditionalUATCommandLine = InAutomatedTest->GetAdditionalUATCommandLine();
	if (!AdditionalUATCommandLine.IsEmpty())
	{
		AdditionalUATCommandLine.ReplaceInline(TEXT("\""), TEXT("\\\""));

		AdditionalCommandLine += TEXT(" ");
		AdditionalCommandLine += AdditionalUATCommandLine;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LauncherProfileManager->OnPostProcessAutomatedTestCommandLine().Broadcast(InAutomatedTest, InProfile, AdditionalCommandLine);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Dispatch the event which may further customize the command line passed to the automated test.
	FUATCommandPostProcessParameters PostProcessParameters =
	{
		.bShouldBeSkipped = true, // skip tests by default unless something explicitly requets it
		.Profile = InProfile,
		.UATCommand = InAutomatedTest,
		.UATCommandLine = AdditionalCommandLine,
		.Commands = OutCommands,
		.CommandStart = OutCommandStart,
	};
	LauncherProfileManager->OnPostProcessUATCommandLine().Broadcast(PostProcessParameters);
	if (PostProcessParameters.bShouldBeSkipped)
	{
		UE_LOGF(LogLauncherProfile, Log, "skipping automated test %ls", InAutomatedTest->GetInternalName() );
		return FString();
	}

	if (!AdditionalCommandLine.IsEmpty())
	{
		AutomatedTestCommand += TEXT(" ");
		AutomatedTestCommand += AdditionalCommandLine;
	}

	return FinalizeCommandLine(AutomatedTestCommand);
}


FString FLauncherWorker::CreateUATCommand( const ILauncherProfileUATCommandRef& InUATCommand, const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms, TArray<FCommandDesc>& OutCommands, FString& OutCommandStart, const FCommandAdjustments& Adjustments )
{
	// skip disabled tests
	if (!InUATCommand->IsEnabled())
	{
		return FString();
	}

	// special case for automated tests
	ILauncherProfileAutomatedTestPtr AutomatedTest = InUATCommand->AsAutomatedTest();
	if (AutomatedTest.IsValid())
	{
		return CreateUATCommandForAutomatedTest(AutomatedTest.ToSharedRef(), InProfile, InPlatforms, OutCommands, OutCommandStart, Adjustments);
	}

	// special case for BuildCookRun
	ILauncherProfileBuildCookRunPtr BuildCookRun = InUATCommand->AsBuildCookRun();
	if (BuildCookRun.IsValid())
	{
		return CreateUATCommandForBuildCookRun(BuildCookRun.ToSharedRef(), InProfile, OutCommands, OutCommandStart, Adjustments);
	}


	// basic command parameters
	FString UATCommandLine = FString::Printf( TEXT("%s "), *InUATCommand->GetUATCommand() );

	FString AdditionalUATCommandLine = InUATCommand->GetAdditionalUATCommandLine();
	if (!AdditionalUATCommandLine.IsEmpty())
	{
		AdditionalUATCommandLine.ReplaceInline(TEXT("\""), TEXT("\\\""));

		UATCommandLine += TEXT(" ");
		UATCommandLine += AdditionalUATCommandLine;
	}

	// Dispatch the event which may further customize the command line passed to the automated test.
	ILauncherProfileManagerRef LauncherProfileManager = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices").GetProfileManager();

	FUATCommandPostProcessParameters PostProcessParameters =
	{
		.bShouldBeSkipped = true, // skip general-purpose UAT commands unless something explicitly requets it
		.Profile = InProfile,
		.UATCommand = InUATCommand,
		.UATCommandLine = UATCommandLine,
		.Commands = OutCommands,
		.CommandStart = OutCommandStart,
	};
	LauncherProfileManager->OnPostProcessUATCommandLine().Broadcast(PostProcessParameters);
	if (PostProcessParameters.bShouldBeSkipped)
	{
		UE_LOGF(LogLauncherProfile, Log, "skipping UAT command %ls", InUATCommand->GetInternalName() );
		return FString();
	}


	return FinalizeCommandLine(UATCommandLine);

}



FString FLauncherWorker::MakeBuildCookRunParamsForProjectCustomBuild(const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms)
{
	// this function can only support one build cook run command (aka "classic launcher profiles")
	TArray<ILauncherProfileBuildCookRunRef> AllBuildCookRunCmds = InProfile->GetBuildCookRunCommands();
	if (AllBuildCookRunCmds.Num() != 1)
	{
		return FString();
	}
	ILauncherProfileBuildCookRunRef BuildCookRun = AllBuildCookRunCmds[0];
	


	// get the basic UAT command line
	FLauncherWorker TempWorker(NoInit);
	TArray<FCommandDesc> Commands;
	FString CommandStart;
	const FCommandAdjustments Adjustments;
	FString BuildCookRunParams = TempWorker.CreateUATCommandForBuildCookRun(AllBuildCookRunCmds[0], InProfile, Commands, CommandStart, Adjustments, true);

	// optional project
	if (InProfile->HasProjectSpecified())
	{
		BuildCookRunParams += FString::Printf( TEXT(" -project=%s"), *FPaths::ConvertRelativePathToFull(InProfile->GetProjectPath()) );
	}
	else
	{
		BuildCookRunParams += TEXT(" -project={project}");
	}

	// optional platform
	if (InPlatforms.Num() == 0)
	{
		BuildCookRunParams += TEXT(" -platform={platform}");
	}

	// optional device
	if ( BuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyToDevice || 
		 BuildCookRun->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch ||
		 BuildCookRun->GetDeployedDeviceGroup() != nullptr )
	{
		BuildCookRunParams += TEXT(" -device={DeviceId}");
	}

	// configuration
	if (BuildCookRun->GetBuildConfiguration() != EBuildConfiguration::Unknown)
	{
		BuildCookRunParams += FString::Printf( TEXT(" -configuration=%s"), LexToString(BuildCookRun->GetBuildConfiguration()) );
	}

	return BuildCookRunParams;
}



ILauncherProfileUATCommandPtr FLauncherWorker::MakeTemporaryUnifiedRunCommand(const ILauncherProfileRef& InProfile)
{
	// find all BuildCookRun commands that want to run that we are able to unify
	TArray<ILauncherProfileBuildCookRunRef> BuildCookRuns;
	for (const ILauncherProfileBuildCookRunRef& BuildCookRun : InProfile->GetBuildCookRunCommands())
	{
		if (BuildCookRun->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch && !BuildCookRun->IsUsingZenPakStreaming() && BuildCookRun->IsEnabled())
		{
			BuildCookRuns.Add(BuildCookRun);
		}
	}

	// nothing to do if there's nothing to unify
	if (BuildCookRuns.Num() < 2)
	{
		return nullptr;
	}

	const TArray<FTargetInfo>& BuildTargets = FDesktopPlatformModule::Get()->GetTargetsForProject(InProfile->GetProjectPath());
	ILauncherProfileManagerRef LauncherProfileManager = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices").GetProfileManager();
	ILauncherProfileBuildCookRunPtr UnifiedRunCommand = MakeShared<FLauncherProfileBuildCookRun>(LauncherProfileManager, nullptr, TEXT("TEMP_UnifiedRunCommand") );


	auto CombineCommandLine = []( const FString& A, const FString& B)
	{
		if (A.IsEmpty())
		{
			return B;
		}
		else if (B.IsEmpty())
		{
			return A;
		}
		else
		{
			return A + TEXT(" ") + B;
		}
	};

	auto GetCommandLine = []( const ILauncherProfileBuildCookRunPtr& BuildCookRun, EBuildTargetType BuildTarget )
	{
		if (BuildTarget == EBuildTargetType::Unknown)
		{
			return BuildCookRun->GetAdditionalCommandLineParameters();
		}
		else
		{
			return BuildCookRun->GetAdditionalTargetCommandLineParameters(BuildTarget);
		}
	};

	auto SetCommandLine = []( const ILauncherProfileBuildCookRunPtr& BuildCookRun, EBuildTargetType BuildTarget, const FString& CommandLine )
	{
		if (BuildTarget == EBuildTargetType::Unknown)
		{
			BuildCookRun->SetAdditionalCommandLineParameters(CommandLine);
		}
		else
		{
			BuildCookRun->SetAdditionalTargetCommandLineParameters(CommandLine, BuildTarget);
		}
	};

	auto MergeCommandLine = [this, UnifiedRunCommand, CombineCommandLine, GetCommandLine, SetCommandLine]( EBuildTargetType BuildTarget, const FString& OtherCommandLine)
	{
		FString CombinedCommandLine = FinalizeCommandLine(CombineCommandLine( GetCommandLine(UnifiedRunCommand, BuildTarget), OtherCommandLine ));
		SetCommandLine( UnifiedRunCommand, BuildTarget, CombinedCommandLine);
	};

	auto GetBuildTargetType = [BuildTargets]( const FString& BuildTargetName )
	{
		for (const FTargetInfo& BuildTarget : BuildTargets)
		{
			if (BuildTarget.Name == BuildTargetName)
			{
				return BuildTarget.Type;
			}
		}

		return EBuildTargetType::Unknown;
	};


	UnifiedRunCommand->SetDeployedDeviceGroup( LauncherProfileManager->CreateUnmanagedDeviceGroup() );

	UnifiedRunCommand->SetDescription(TEXT("Unified Run"));
	UnifiedRunCommand->SetBuildMode(ELauncherProfileBuildModes::DoNotBuild);
	UnifiedRunCommand->SetCookMode(ELauncherProfileCookModes::DoNotCook);
	UnifiedRunCommand->SetPackagingMode(ELauncherProfilePackagingModes::DoNotPackage);
	UnifiedRunCommand->SetDeploymentMode(ELauncherProfileDeploymentModes::DoNotDeploy);
	UnifiedRunCommand->SetLaunchMode(ELauncherProfileLaunchModes::DefaultRole);
	UnifiedRunCommand->SetAdditionalUATCommandLine(TEXT(" -skipbuild -skipcook -skipstage -skipdeploy")); // @todo: check if any of these are implied by the items above

	bool bFirstItem = true;
	for ( const ILauncherProfileBuildCookRunRef& BuildCookRun : BuildCookRuns)
	{
		// unify platforms & targets
		for (const FString& CookedPlatform : BuildCookRun->GetCookedPlatforms())
		{
			UnifiedRunCommand->AddCookedPlatform(CookedPlatform);
		}

		for (const FString& BuildTarget : BuildCookRun->GetBuildTargets())
		{
			UnifiedRunCommand->AddBuildTarget(BuildTarget);
			UnifiedRunCommand->SetBuildTargetSpecified(true);
		}

		// unify architectures
		TArray<FString> ClientArchitectures;
		for (const FString& ClientArchitecture : BuildCookRun->GetClientArchitectures())
		{
			ClientArchitectures.AddUnique(ClientArchitecture);
		}
		UnifiedRunCommand->SetClientArchitectures(ClientArchitectures);

		TArray<FString> ServerArchitectures;
		for (const FString& ServerArchitecture : BuildCookRun->GetServerArchitectures())
		{
			ServerArchitectures.AddUnique(ServerArchitecture);
		}
		UnifiedRunCommand->SetServerArchitectures(ServerArchitectures);

		TArray<FString> EditorArchitectures;
		for (const FString& EditorArchitecture : BuildCookRun->GetEditorArchitectures())
		{
			EditorArchitectures.AddUnique(EditorArchitecture);
		}
		UnifiedRunCommand->SetEditorArchitectures(EditorArchitectures);

		// unify launch command lines
		EBuildTargetType SingleBuildTarget = (BuildCookRun->GetBuildTargets().Num() == 1) ? GetBuildTargetType(BuildCookRun->GetBuildTargets()[0]) : EBuildTargetType::Unknown;

		const EBuildTargetType SupportedBuildTargets[] = { EBuildTargetType::Unknown, EBuildTargetType::Client, EBuildTargetType::Server, EBuildTargetType::Game, EBuildTargetType::Editor };
		for (const EBuildTargetType BuildTarget : SupportedBuildTargets)
		{
			// copy the per-target type command line over - only if the build target matches, or the source has many targets anyway
			if (SingleBuildTarget == EBuildTargetType::Unknown || SingleBuildTarget == BuildTarget)
			{
				MergeCommandLine( BuildTarget, GetCommandLine(BuildCookRun, BuildTarget) );
			}

			// copy the target-agnostic command line over into the target-specific command line property
			if (SingleBuildTarget != EBuildTargetType::Unknown && BuildTarget == EBuildTargetType::Unknown)
			{
				MergeCommandLine( SingleBuildTarget, GetCommandLine(BuildCookRun, BuildTarget) );
			}

		}

		// unify devices
		for ( const FString& DeviceId : BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs())
		{
			UnifiedRunCommand->GetDeployedDeviceGroup()->AddDevice(DeviceId);
		}

		// first come, first served on the initial map
		if (UnifiedRunCommand->GetDefaultLaunchRole()->GetInitialMap().IsEmpty())
		{
			UnifiedRunCommand->GetDefaultLaunchRole()->SetInitialMap(BuildCookRun->GetDefaultLaunchRole()->GetInitialMap());
		}

		// get consensus on using a pre-staged build
		if (bFirstItem && BuildCookRun->IsUsingPreStagedBuild())
		{
			UnifiedRunCommand->SetUsePreStagedBuild(true);
		}
		else if (!bFirstItem && BuildCookRun->IsUsingPreStagedBuild() != UnifiedRunCommand->IsUsingPreStagedBuild())
		{
			// @todo: surface this better
			UE_LOGF(LogLauncherProfile, Warning, "Cannot create unified build cook run because not all builds agree on whether to use a pre-staged build");
			return nullptr;
		}

		// ... @todo: any other properties that need unification? ...


		bFirstItem = false;
	}

	return UnifiedRunCommand;
}

TArray<FString> FLauncherWorker::FinalizePlatforms( const TArray<FString>& CookPlatforms, const ILauncherDeviceGroupPtr& DeviceGroup, bool bPreferCookedPlatforms ) const
{
	TArray<FString> Platforms;
	if (bPreferCookedPlatforms)
	{
		Platforms = CookPlatforms;
	}

	// determine deployment platforms
	FName Variant = NAME_None;

	if (DeviceGroup.IsValid() && Platforms.Num() < 1)
	{
		const TArray<FString>& Devices = DeviceGroup->GetDeviceIDs();
		// for each deployed device...
		for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
		{
			const FString& DeviceId = Devices[DeviceIndex];

			TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceId);

			if (DeviceProxy.IsValid())
			{
				// add the platform
				Variant = DeviceProxy->GetTargetDeviceVariant(DeviceId);
				Platforms.AddUnique(DeviceProxy->GetTargetPlatformName(Variant));
			}			
		}
	}

	// could not get a platform from the device proxy (suggesting to use the default device) so use the selected cook platforms
	if (Platforms.Num() == 0)
	{
		Platforms = CookPlatforms;
	}

	return MoveTemp(Platforms);
}

FString FLauncherWorker::FinalizeCommandLine( const FString& CommandLine ) const
{
	const TCHAR* CommandLinePtr = CommandLine.GetCharArray().GetData();
	
	TArray<FString> Result;

	FString Token;
	while( CommandLinePtr != nullptr && FParse::Token(CommandLinePtr, Token, false) )
	{
		Result.AddUnique(Token);
	}

	return FString::Join(Result, TEXT(" "));
}



/* FLauncherWorker implementation
 *****************************************************************************/

void FLauncherWorker::CreateAndExecuteTasks( const ILauncherProfileRef& InProfile )
{
	InputPipe = UE::HAL::NewPipe;

	TArray<FString> Platforms;
	TArray<FString> DeviceIDs;
	for (const ILauncherProfileBuildCookRunRef& BuildCookRun : InProfile->GetBuildCookRunCommands())
	{
		bool bPreferCookedPlatforms = BuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook || BuildCookRun->ShouldBuild();
		TArray<FString> FinalPlatforms = FinalizePlatforms(BuildCookRun->GetCookedPlatforms(), BuildCookRun->GetDeployedDeviceGroup(), bPreferCookedPlatforms);
		for (const FString& Platform : FinalPlatforms)
		{
			Platforms.AddUnique(Platform);
		}

		for (const FString& DeviceID : BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs())
		{
			DeviceIDs.AddUnique(DeviceID);
		}

#if !WITH_EDITOR
		// can't cook by the book in the editor if we are not in the editor...
		check( BuildCookRun->GetCookMode() != ELauncherProfileCookModes::ByTheBookInEditor );
		check( BuildCookRun->GetCookMode() != ELauncherProfileCookModes::OnTheFlyInEditor );
#endif
	}

	// create task chains
	TSharedPtr<FLauncherTask> NextTask;
	auto AddTask = [this, &NextTask](TSharedPtr<FLauncherTask> NewTask, ILauncherProfileUATCommandPtr UnifiedUATCommand = nullptr)
	{
		NewTask->OnStarted().AddRaw(this, &FLauncherWorker::OnTaskStarted, UnifiedUATCommand);
		NewTask->OnCompleted().AddRaw(this, &FLauncherWorker::OnTaskCompleted);

		if (NextTask.IsValid())
		{
			NextTask->AddContinuation(NewTask);
			NextTask = NewTask;
		}
		else
		{
			NextTask = TaskChain = NewTask;
		}
	};

	if (InProfile->GetFirstBuildCookRun().IsValid() && InProfile->GetFirstBuildCookRun()->GetCookMode() == ELauncherProfileCookModes::ByTheBookInEditor)
	{
		// need a command which will wait for the cook to finish
		struct FWaitForCookInEditorToFinish: FLauncherTask
		{
			FWaitForCookInEditorToFinish()
				: FLauncherTask{TEXT("Cooking in the editor"), TEXT("Preparing content to run on device")}
			{
			}

		protected:
			virtual bool PerformTask(const FLauncherTaskChainState& InChainState) override
			{
				while ( !InChainState.Profile->OnIsCookFinished().Execute() )
				{
					if (IsCancelling())
					{
						InChainState.Profile->OnCookCanceled().Execute();
						return false;
					}
					FPlatformProcess::Sleep( 0.1f );
				}
				return true;
			}
		};
		AddTask(MakeShared<FWaitForCookInEditorToFinish>());
	}

	// gather all ordered UAT commands
	TArray<ILauncherProfileUATCommandRef> UATCommands = InProfile->GetUATCommands();
	UATCommands.Sort( []( const ILauncherProfileUATCommandRef& A, const ILauncherProfileUATCommandRef& B)
	{
		return A->GetOrder() < B->GetOrder();
	});
	int UATCommandIndex = 0;


	FCommandAdjustments CommandAdjustments;
	TArray<FCommandDesc> Commands;
	FString StartString;
	FString UATCommandString;

	// special case for Turnkey
	if (Profile->ShouldUpdateDeviceFlash() && DeviceIDs.Num() > 0)
	{
		UATCommandString += FString::Printf(TEXT("Turnkey -command=VerifySdk -type=Flash -device=%s -UpdateIfNeeded -WaitForUATMutex %s "), *FString::Join(DeviceIDs, TEXT("+")), *ITurnkeyIOModule::Get().GetUATParams());
	}

	
	ILauncherProfileUATCommandPtr UnifiedRunUATCommand;
	if (InProfile->GetAutomatedTests().Num() > 0)
	{
		// disable run in BuildCookRun commands if there are automated tests as they will take care of the run
		CommandAdjustments.bDisableRun = true;
	}
	else
	{
		// disable run in BuildCookRun commands if there is multiple BuildCookRun commands that want to run - in this case, they're gathered together and run in a separate unified run command at the end
		UnifiedRunUATCommand = MakeTemporaryUnifiedRunCommand(InProfile);
		if (UnifiedRunUATCommand.IsValid())
		{
			UATCommands.Add(UnifiedRunUATCommand.ToSharedRef());
			CommandAdjustments.bDisableRun = true;
		}
	}


	// build all UAT commands
	for (const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
	{
		FCommandAdjustments Adjustments = CommandAdjustments;
		if (UATCommand == UnifiedRunUATCommand)
		{
			Adjustments.bDisableRun = false;
			Adjustments.bPreferCookedPlatforms = true;
		}

		UATCommandString += CreateUATCommand(UATCommand, InProfile, Platforms, Commands, StartString, Adjustments);
		UATCommandString += TEXT(" ");
	}


	// wait for completion of automated tests
	if (Profile->GetAutomatedTests().Num() > 0)
	{
		FCommandDesc Desc;
		FText Command = LOCTEXT("LauncherAutoTestDesc", "Automated Testing");
		Desc.Name = "Automated Testing Task";
		Desc.Desc = Command.ToString();
		Desc.EndText = TEXT("********** AUTOMATED TESTS COMPLETED **********"); // @note: this isn't actually logged anywhere - this is the last real task so it will advance automatically
		Commands.Add(Desc);
	}


	// wait for completion of UAT
	{
		FCommandDesc Desc;
		FText Command = LOCTEXT("LauncherCompletionDesc", "UAT post launch cleanup");
		Desc.Name = "Post Launch Task";
		Desc.Desc = Command.ToString();
		Desc.EndText = TEXT("********** LAUNCH COMPLETED **********");
		Commands.Add(Desc);
	}

	TSharedPtr<FLauncherTask> LaunchTask = MakeShareable(new FLauncherUATTask(UATCommandString, TEXT("Launch Task"), TEXT("Launching UAT..."), InputPipe, Process, this, StartString));
	AddTask(LaunchTask, UnifiedRunUATCommand);
	for (int32 Index = 0; Index < Commands.Num(); ++Index)
	{
		struct FLauncherWaitTask: FLauncherTask
		{
			FLauncherWaitTask(FString InCommandEnd, FString InName, FString InDesc, UE::HAL::FProcess& InProcess, ILauncherWorker* InWorker)
				: FLauncherTask{MoveTemp(InName), MoveTemp(InDesc)}
				, CommandText{MoveTemp(InCommandEnd)}
				, Process{InProcess}
				, LauncherWorker{InWorker}
			{
				InWorker->OnOutputReceived().AddRaw(this, &FLauncherWaitTask::HandleOutputReceived);
			}

			virtual void Exit()
			{
				LauncherWorker->OnOutputReceived().RemoveAll(this);
			}

		protected:
			virtual bool PerformTask(const FLauncherTaskChainState& InChainState) override
			{
				while (!EndTextFound.load(std::memory_order_relaxed))
				{
					if (!Process.IsRunning())
					{
						Result = *Process.GetExitCode();
						break;
					}

					FPlatformProcess::Sleep(0.25);
				}

				return Result == 0;
			}

			void HandleOutputReceived(const FString& InMessage)
			{
				EndTextFound.store(EndTextFound.load(std::memory_order_relaxed) || InMessage.Contains(CommandText), std::memory_order_relaxed);
			}

		private:
			FString CommandText;
			UE::HAL::FProcess& Process;
			ILauncherWorker* LauncherWorker = nullptr;
			std::atomic_bool EndTextFound = false;
		};			
		AddTask(MakeShared<FLauncherWaitTask>(Commands[Index].EndText, Commands[Index].Name, Commands[Index].Desc, Process, this));
	}

	// execute the chain
	ChainState.Profile = InProfile;
	ChainState.SessionId = FGuid::NewGuid();
	TaskChain->Execute(&ChainState);
}

/** Callback for when a device proxy has been removed from the device proxy manager. */
void FLauncherWorker::HandleDeviceProxyManagerProxyRemoved(const TSharedRef<ITargetDeviceProxy>& RemovedProxy, ILauncherProfileUATCommandPtr UnifiedUATCommand)
{
	TArray<FName> Variants;
	RemovedProxy->GetVariants(Variants);

	TSet<FString> TargetDeviceIds;
	for (FName Variant : Variants)
	{
		TargetDeviceIds.Append( RemovedProxy->GetTargetDeviceIds(Variant) );
	}

	TArray<ILauncherProfileUATCommandRef> UATCommands = Profile->GetUATCommands();
	if (UnifiedUATCommand.IsValid())
	{
		UATCommands.Add(UnifiedUATCommand.ToSharedRef());
	}

	for (const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
	{
		if (!UATCommand->IsEnabled())
		{
			continue;
		}

		if (ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun())
		{
			if (BuildCookRun->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch && (!UnifiedUATCommand.IsValid() || UnifiedUATCommand == UATCommand))
			{
				ILauncherDeviceGroupPtr DeviceGroup = BuildCookRun->GetDeployedDeviceGroup();

				if (DeviceGroup.IsValid() && DeviceGroup->GetNumDevices() > 0)
				{
					// remove disconnected device from the list
					for (const FString& TargetDeviceId : TargetDeviceIds)
					{
						DeviceGroup->RemoveDevice(TargetDeviceId);
						if (DeviceGroup->GetNumDevices() == 0)
						{
							// this was the last device running, stop working
							Stop();
							break;
						}
					}
				}
			}
		}
	}
}

// start looking for disconnected devices
void FLauncherWorker::EnableDeviceDiscoveryListener(ILauncherProfileUATCommandPtr UnifiedUATCommand)
{
	if (!OnProxyRemovedDelegateHandle.IsValid())
	{
		OnProxyRemovedDelegateHandle = DeviceProxyManager->OnProxyRemoved().AddRaw(this, &FLauncherWorker::HandleDeviceProxyManagerProxyRemoved, UnifiedUATCommand);
	}
}

// stop looking for disconnected devices
void FLauncherWorker::DisableDeviceDiscoveryListener()
{
	if (OnProxyRemovedDelegateHandle.IsValid())
	{
		DeviceProxyManager->OnProxyRemoved().Remove(OnProxyRemovedDelegateHandle);
		OnProxyRemovedDelegateHandle.Reset();
	}

}

//Cancel the currently running application on all devices
bool FLauncherWorker::TerminateLaunchedProcess()
{
	for (const ILauncherProfileUATCommandRef& UATCommand : Profile->GetUATCommands())
	{
		TArray<FString> Devices;
		if (const ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun())
		{
			// determine deployment devices
			ILauncherDeviceGroupPtr DeviceGroup = BuildCookRun->GetDeployedDeviceGroup();
			FName Variant = NAME_None;

			// device list
			FString DeviceNamesParam = TEXT("");
			if (DeviceGroup.IsValid())
			{
				Devices = DeviceGroup->GetDeviceIDs();
			}
		}

		// cancel the app onr each device
		for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
		{
			const FString& DeviceId = Devices[DeviceIndex];

			TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceId);
			if (DeviceProxy.IsValid())
			{
				FName TargetDeviceVariant = DeviceProxy->GetTargetDeviceVariant(DeviceId);

				FString TargetDeviceId = DeviceId;
				
				// remove the variant prefix (eg. Android_ETC@deviceId)
				int32 InPos = TargetDeviceId.Find("@", ESearchCase::CaseSensitive);
				if (InPos > 0) 
				{ 
					TargetDeviceId.RightInline(TargetDeviceId.Len() -  InPos - 1, EAllowShrinking::No);

				}

				// try to find the corresponding app id
				if (CachedDevicePackagePair.Contains(TargetDeviceId))
				{
					DeviceProxy->TerminateLaunchedProcess(TargetDeviceVariant, CachedDevicePackagePair[TargetDeviceId]);
				}
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
