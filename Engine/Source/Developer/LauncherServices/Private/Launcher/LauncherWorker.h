// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherWorker.h"
#include "LauncherTaskChainState.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

class FLauncherTask;
class ITargetDeviceProxyManager;
class ITargetDeviceProxy;

/**
 * Implements the launcher's worker thread.
 */
class FLauncherWorker
	: public FRunnable
	, public ILauncherWorker
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InDeviceProxyManager The target device proxy manager to use.
	 * @param InProfile The profile to process.
	 * @param InWorkerName The name of the worker thread.
	 */
	FLauncherWorker(const TSharedRef<ITargetDeviceProxyManager>& InDeviceProxyManager, const ILauncherProfileRef& InProfile, const TCHAR* InWorkerName);

	virtual ~FLauncherWorker() override
	{
		if (Thread)
		{
			Thread->WaitForCompletion();
		}

		TaskChain.Reset();
		DisableDeviceDiscoveryListener();
	}
	
	/** Returns whether the worker is valid to run. */
	bool IsValid() const
	{
		return bool(Thread);
	}

public:

	virtual bool Init( ) override;

	virtual uint32 Run( ) override;

	virtual void Stop( ) override;

	virtual void Exit( ) override { }

public:

	virtual void Cancel( ) override;

	virtual void CancelAndWait( ) override;

	virtual ELauncherWorkerStatus::Type GetStatus() const override
	{
		return Status.load(std::memory_order_relaxed);
	}

	virtual int32 GetTasks( TArray<ILauncherTaskPtr>& OutTasks ) const override;

	virtual FOutputMessageReceivedDelegate& OnOutputReceived() override
	{
		return OutputMessageReceived;
	}

	virtual FOnStageStartedDelegate& OnStageStarted() override
	{
		return StageStarted;
	}

	virtual FOnStageCompletedDelegate& OnStageCompleted() override
	{
		return StageCompleted;
	}

	virtual FOnLaunchCompletedDelegate& OnCompleted() override
	{
		return LaunchCompleted;
	}

	virtual FOnLaunchCanceledDelegate& OnCanceled() override
	{
		return LaunchCanceled;
	}

	virtual ILauncherProfilePtr GetLauncherProfile() const override
	{
		return Profile;
	}

	/** Callback for when a device proxy has been removed from the device proxy manager. */
	void HandleDeviceProxyManagerProxyRemoved(const TSharedRef<ITargetDeviceProxy>& RemovedProxy, ILauncherProfileUATCommandPtr UnifiedUATCommand);

	/**
	 * Set the app id running on the device
	 */
	virtual void AddDevicePackagePair(const FString& Device, const FString& Package) override
	{
		CachedDevicePackagePair.Add(Device, Package);
	}

	/**
	 * Returns best-guess BuildCookRun parameters for the given profile in the format expected by FProjectBuidSettings::BuildCookRunParams
	 *  Used by the Project custom Build's "import from Project Launcher" menu
	 */
	static FString MakeBuildCookRunParamsForProjectCustomBuild(const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms);

protected:

	/**
	 * Creates the tasks for the specified profile.
	 *
	 * @param InProfile - The profile to create tasks for.
	 */
	void CreateAndExecuteTasks( const ILauncherProfileRef& InProfile );

	void OnTaskStarted(const FString& TaskName, ILauncherProfileUATCommandPtr UnifiedUATCommand);
	void OnTaskCompleted(const FString& TaskName);

	struct FCommandAdjustments
	{
		bool bDisableRun = false;
		bool bPreferCookedPlatforms = false;
	};
	FString CreateUATCommandForBuildCookRun( const ILauncherProfileBuildCookRunRef& InBuildCookRun, const ILauncherProfileRef& InProfile, TArray<FCommandDesc>& OutCommands, FString& OutCommandStart, const FCommandAdjustments& Adjustments, bool bForTurnkeyCustomBuild = false );

	FString CreateUATCommand( const ILauncherProfileUATCommandRef& InUATCommand, const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms, TArray<FCommandDesc>& OutCommands, FString& OutCommandStart, const FCommandAdjustments& Adjustments );

	FString CreateUATCommandForAutomatedTest( const ILauncherProfileAutomatedTestRef& InAutomatedTest, const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms, TArray<FCommandDesc>& OutCommands, FString& OutCommandStart, const FCommandAdjustments& Adjustments );

	ILauncherProfileUATCommandPtr MakeTemporaryUnifiedRunCommand(const ILauncherProfileRef& InProfile);
	TArray<FString> FinalizePlatforms( const TArray<FString>& CookPlatforms, const ILauncherDeviceGroupPtr& DeviceGroup, bool bPreferCookedPlatforms ) const;
	FString FinalizeCommandLine( const FString& CommandLine ) const;

	//start listening to the device discovery routine
	void EnableDeviceDiscoveryListener(ILauncherProfileUATCommandPtr UnifiedUATCommand);

	//stop listening to the device discovery routine
	void DisableDeviceDiscoveryListener();

	//Cancel the currently running application on all devices
	bool TerminateLaunchedProcess();

	// just used by MakeBuildCookRunParamsForProjectCustomBuild
	FLauncherWorker(ENoInit) {};

private:

	// Holds a critical section to lock access to selected properties.
	FCriticalSection CriticalSection;

	// Holds a pointer  to the device proxy manager.
	TSharedPtr<ITargetDeviceProxyManager> DeviceProxyManager;

	// Holds a pointer to the launcher profile.
	ILauncherProfilePtr Profile;

	// Holds the worker's current status.
	std::atomic<ELauncherWorkerStatus::Type> Status = ELauncherWorkerStatus::Busy;

	// Holds the first task in the task chain.
	TSharedPtr<FLauncherTask> TaskChain;
	FLauncherTaskChainState ChainState;

	TUniquePtr<FRunnableThread> Thread;

	// Holds the input pipe and the running UAT process.
	UE::HAL::FInputPipe InputPipe;
	UE::HAL::FProcess Process;

	double StageStartTime = 0.0;
	double LaunchStartTime = 0.0;

	// message delegate
	FOutputMessageReceivedDelegate OutputMessageReceived;
	FOnStageStartedDelegate StageStarted;
	FOnStageCompletedDelegate StageCompleted;
	FOnLaunchCompletedDelegate LaunchCompleted;
	FOnLaunchCanceledDelegate LaunchCanceled;

	// delegate for device discovery (device removed)
	FDelegateHandle OnProxyRemovedDelegateHandle;

	// Dictionary holding the appid@device pairs from the last launch
	TMap<FString, FString> CachedDevicePackagePair;
};
