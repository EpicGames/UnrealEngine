// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"
#include "ILauncherServicesModule.h"
#include "Profiles/LauncherDeviceGroup.h"
#include "Profiles/LauncherProfile.h"
#include "Profiles/LauncherProfileManager.h"
#include "Launcher/Launcher.h"


/**
 * Implements the LauncherServices module.
 */
class FLauncherServicesModule
	: public ILauncherServicesModule
{
public:

	//~ ILauncherServicesModule interface

	virtual ILauncherDeviceGroupRef CreateDeviceGroup() override
	{
		return MakeShareable(new FLauncherDeviceGroup());
	}

	virtual ILauncherDeviceGroupRef CreateDeviceGroup(const FGuid& Guid, const FString& Name) override
	{
		return MakeShareable(new FLauncherDeviceGroup(Guid, Name));
	}

	virtual ILauncherRef CreateLauncher() override
	{
		ILauncherRef Launcher = MakeShareable(new FLauncher());
		OnCreateLauncherDelegate.Broadcast(Launcher);
		return Launcher;
	}

	virtual ILauncherProfileRef CreateProfile(const FString& ProfileName) override
	{
		ILauncherProfileManagerRef ProfileManager = GetProfileManager();
		TSharedRef<FLauncherProfile> NewProfile = MakeShared<FLauncherProfile>(ProfileManager, FGuid(), ProfileName);
		NewProfile->SetDefaults();

		return NewProfile;
	}

	virtual ILauncherProfileManagerRef GetProfileManager() override
	{
		if (!ProfileManagerSingleton.IsValid())
		{
			TSharedPtr<FLauncherProfileManager> ProfileManager = MakeShareable(new FLauncherProfileManager());	
			ProfileManager->Load();
			ProfileManagerSingleton = ProfileManager;
			ProfileManagerInitializedDelegate.Broadcast(*ProfileManager);
		}

		return ProfileManagerSingleton.ToSharedRef();
	}
	
	DECLARE_DERIVED_EVENT(FLauncherServicesModule, ILauncherServicesModule::FLauncherServicesSDKNotInstalled, FLauncherServicesSDKNotInstalled);
	virtual FLauncherServicesSDKNotInstalled& OnLauncherServicesSDKNotInstalled() override
	{
		return LauncherServicesSDKNotInstalled;
	}
	void BroadcastLauncherServicesSDKNotInstalled(const FString& PlatformName, const FString& DocLink) override
	{
		return LauncherServicesSDKNotInstalled.Broadcast(PlatformName, DocLink);
	}

	virtual FString GetExecutableForCommandlets() const
	{
		FString ExecutableName;
#if WITH_EDITOR
		ExecutableName = FString(FPlatformProcess::ExecutablePath());
#if PLATFORM_WINDOWS
		// turn UnreadEditor into UnrealEditor-cmd
		if (ExecutableName.EndsWith(".exe", ESearchCase::IgnoreCase) && !FPaths::GetBaseFilename(ExecutableName).EndsWith("-cmd", ESearchCase::IgnoreCase))
		{
			FString NewExeName = ExecutableName.Left(ExecutableName.Len() - 4) + "-Cmd.exe";
			if (FPaths::FileExists(NewExeName))
			{
				ExecutableName = NewExeName;
			}
		}
#elif PLATFORM_MAC
		// turn UnrealEditor into UnrealEditor-cmd
		if (!FPaths::GetBaseFilename(ExecutableName).EndsWith("-cmd", ESearchCase::IgnoreCase))
		{
			FString NewExeName = ExecutableName + "-Cmd";
			if (FPaths::FileExists(NewExeName))
			{
				ExecutableName = NewExeName;
			}
		}
#endif
#endif
		return ExecutableName;

	}

private:
	
	/** Event to be called when the editor tried to use a platform, but it wasn't installed. */
	FLauncherServicesSDKNotInstalled LauncherServicesSDKNotInstalled;

	/** The launcher profile manager singleton. */
	static ILauncherProfileManagerPtr ProfileManagerSingleton;
};


/* Static initialization
 *****************************************************************************/

FOnLauncherProfileManagerInitialized ILauncherServicesModule::ProfileManagerInitializedDelegate;
ILauncherProfileManagerPtr FLauncherServicesModule::ProfileManagerSingleton = NULL;


IMPLEMENT_MODULE(FLauncherServicesModule, LauncherServices);
