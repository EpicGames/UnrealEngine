// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ILauncherDeviceGroup.h"

/**
 * Implements a group of devices for the Launcher user interface.
 */
class FLauncherDeviceGroup
	: public TSharedFromThis<FLauncherDeviceGroup>
	, public ILauncherDeviceGroup
{
public:

	/**
	 * Default constructor.
	 */
	FLauncherDeviceGroup() { }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InId - The unique group identifier.
	 * @param InName - The name of the group.
	 */
	FLauncherDeviceGroup(const FGuid& InId, const FString& InName)
		: Id(InId)
		, Name(InName)
	{ }

public:

	//~ Begin ILauncherDeviceGroup Interface

	virtual void AddDevice(const FString& DeviceID) override
	{
		Devices.AddUnique(DeviceID);

		DeviceAddedDelegate.Broadcast(AsShared(), DeviceID);
	}

	virtual const TArray<FString>& GetDeviceIDs() override
	{
		return Devices;
	}

	virtual FGuid GetId() const override
	{
		return Id;
	}

	virtual const FString& GetName() const override
	{
		return Name;
	}

	virtual int32 GetNumDevices() const override
	{
		return Devices.Num();
	}

	virtual FOnLauncherDeviceGroupDeviceAdded& OnDeviceAdded() override
	{
		return DeviceAddedDelegate;
	}

	virtual FOnLauncherDeviceGroupDeviceRemoved& OnDeviceRemoved() override
	{
		return DeviceRemovedDelegate;
	}

	virtual void RemoveDevice(const FString& DeviceID) override
	{
		Devices.Remove(DeviceID);

		DeviceRemovedDelegate.Broadcast(AsShared(), DeviceID);
	}

	virtual void RemoveAllDevices() override
	{
		for (int32 i = Devices.Num() - 1; i >= 0; --i)
		{
			FString DeviceId = Devices[i];
			Devices.RemoveAt(i, EAllowShrinking::No);

			DeviceRemovedDelegate.Broadcast(AsShared(), DeviceId);
		}
	}

	virtual void SetName(const FString& NewName) override
	{
		Name = NewName;
	}

	//~ End ILauncherDeviceGroup Interface

private:

	// Holds the devices that are part of this group.
	TArray<FString> Devices;

	// Holds the group's unique identifier.
	FGuid Id;

	// Holds the human readable name of this group.
	FString Name;

private:

	// Holds a delegate to be invoked when a device was added to this group.
	FOnLauncherDeviceGroupDeviceAdded DeviceAddedDelegate;

	// Holds a delegate to be invoked when a device was removed from this group.
	FOnLauncherDeviceGroupDeviceRemoved DeviceRemovedDelegate;
};
