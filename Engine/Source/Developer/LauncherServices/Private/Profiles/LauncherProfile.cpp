// Copyright Epic Games, Inc. All Rights Reserved.

#include "LauncherProfile.h"
#include "Profiles/LauncherProfileUATCommand.h"
#include "Profiles/LauncherProfileAutomatedTest.h"
#include "Profiles/LauncherProfileBuildCookRun.h"

#define LOCTEXT_NAMESPACE "SProjectLauncherValidation"

DEFINE_LOG_CATEGORY(LogLauncherProfile);

const TCHAR* FLauncherProfileUATCommand::TypeName = TEXT("UATCommand");
const TCHAR* FLauncherProfileAutomatedTest::TypeName = TEXT("AutomatedTest");
const TCHAR* FLauncherProfileBuildCookRun::TypeName = TEXT("BuildCookRun");


ILauncherProfileUATCommandRef FLauncherProfile::CreateUATCommandInternal(const TCHAR* InternalName, const TCHAR* UserTypeName)
{
	ILauncherProfileUATCommandRef UATCommand = MakeShared<FLauncherProfileUATCommand>(AsShared(), InternalName, UserTypeName);
	return UATCommand;
}

ILauncherProfileAutomatedTestRef FLauncherProfile::CreateAutomatedTestInternal(const TCHAR* InternalName, const TCHAR* UserTypeName)
{
	ILauncherProfileAutomatedTestRef AutomatedTest = MakeShared<FLauncherProfileAutomatedTest>(AsShared(), InternalName, UserTypeName);
	return AutomatedTest;
}

ILauncherProfileBuildCookRunRef FLauncherProfile::CreateBuildCookRunInternal(const TCHAR* InternalName, const TCHAR* UserTypeName)
{
	ILauncherProfileBuildCookRunRef BuildCookRun = MakeShared<FLauncherProfileBuildCookRun>(LauncherProfileManager, AsShared(), InternalName, UserTypeName);
	return BuildCookRun;

}

ILauncherProfileUATCommandPtr FLauncherProfile::CreateUATCommandByTypeNameInternal( const TCHAR* TypeName, const TCHAR* InternalName )
{
	const TCHAR* UserTypeName = TEXT("");
	if (FCString::Stricmp(TypeName, FLauncherProfileUATCommand::TypeName) == 0)
	{
		return CreateUATCommandInternal(InternalName, UserTypeName);
	}
	else if (FCString::Stricmp(TypeName, FLauncherProfileAutomatedTest::TypeName) == 0)
	{
		return CreateAutomatedTestInternal(InternalName, UserTypeName);
	}
	else if (FCString::Stricmp(TypeName, FLauncherProfileBuildCookRun::TypeName) == 0)
	{
		return CreateBuildCookRunInternal(InternalName, UserTypeName);
	}
	else
	{
		return nullptr;
	}
}


ILauncherProfileBuildCookRunPtr FLauncherProfile::LoadBuildCookRunInternal(const FJsonObject& Object, int32 Version)
{
	TSharedRef<FLauncherProfileBuildCookRun> BuildCookRun = MakeShared<FLauncherProfileBuildCookRun>(LauncherProfileManager, AsShared(), TEXT("OriginalBuildCookRun"));
	if (!BuildCookRun->Load(Object, Version))
	{
		return nullptr;
	}

	// should always have a device group
	ILauncherDeviceGroupRef DeployDeviceGroup = LauncherProfileManager->AddNewDeviceGroup();
	BuildCookRun->SetDeployedDeviceGroup(DeployDeviceGroup);

	return BuildCookRun;
}




FString LexToStringLocalized(ELauncherProfileValidationErrors::Type Value)
{
	static_assert(ELauncherProfileValidationErrors::Count == 31, "GetLocalizedValidationErrorMessage() needs to be updated to account for modified enum values.");
	switch (Value)
	{
		case ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook:
			return LOCTEXT("CopyToDeviceRequiresCookByTheBookError", "Deployment by copying to device requires cooking, zen streaming or a pre-staged build.").ToString();
		case ELauncherProfileValidationErrors::CustomRolesNotSupportedYet:
			return LOCTEXT("CustomRolesNotSupportedYet", "Custom launch roles are not supported yet.").ToString();
		case ELauncherProfileValidationErrors::DeployedDeviceGroupRequired:
			return LOCTEXT("DeployedDeviceGroupRequired", "A device group must be selected when deploying builds.").ToString();
		case ELauncherProfileValidationErrors::InitialCultureNotAvailable:
			return LOCTEXT("InitialCultureNotAvailable", "The Initial Culture selected for launch is not in the build.").ToString();
		case ELauncherProfileValidationErrors::InitialMapNotAvailable:
			return LOCTEXT("InitialMapNotAvailable", "The Initial Map selected for launch is not in the build.").ToString();
		case ELauncherProfileValidationErrors::MalformedLaunchCommandLine:
			return LOCTEXT("MalformedLaunchCommandLine", "The specified launch command line is not formatted correctly.").ToString();
		case ELauncherProfileValidationErrors::NoBuildConfigurationSelected:
			return LOCTEXT("NoBuildConfigurationSelectedError", "A Build Configuration must be selected.").ToString();
		case ELauncherProfileValidationErrors::NoCookedCulturesSelected:
			return LOCTEXT("NoCookedCulturesSelectedError", "At least one Culture must be selected when cooking.").ToString();
		case ELauncherProfileValidationErrors::NoLaunchRoleDeviceAssigned:
			return LOCTEXT("NoLaunchRoleDeviceAssigned", "One or more launch roles do not have a device assigned.").ToString();
		case ELauncherProfileValidationErrors::NoPlatformSelected:
			return LOCTEXT("NoCookedPlatformSelectedError", "At least one Platform must be selected.").ToString();
		case ELauncherProfileValidationErrors::NoProjectSelected:
			return LOCTEXT("NoBuildGameSelectedError", "A Project must be selected.").ToString();
		case ELauncherProfileValidationErrors::NoPackageDirectorySpecified:
			return LOCTEXT("NoPackageDirectorySpecified", "The deployment requires a package directory to be specified.").ToString();
		case ELauncherProfileValidationErrors::NoPlatformSDKInstalled:
			return LOCTEXT("NoPlatformSDKInstalled", "A required platform SDK is missing.").ToString();
		case ELauncherProfileValidationErrors::UnversionedAndIncremental:
			return LOCTEXT("UnversionedAndIncremental", "Unversioned build cannot be incremental when using LegacyIterative cook setting.").ToString();
		case ELauncherProfileValidationErrors::GeneratingPatchesCanOnlyRunFromByTheBookCookMode:
			return LOCTEXT("GeneratingPatchesCanOnlyRunFromByTheBookCookMode", "Generating patch requires cooking or zen snapshot import.").ToString();
		case ELauncherProfileValidationErrors::GeneratingMultiLevelPatchesRequiresGeneratePatch:
			return LOCTEXT("GeneratingMultiLevelPatchesRequiresGeneratePatch", "Generating multilevel patch requires generating patch.").ToString();
		case ELauncherProfileValidationErrors::StagingBaseReleasePaksWithoutABaseReleaseVersion:
			return LOCTEXT("StagingBaseReleasePaksWithoutABaseReleaseVersion", "Staging base release pak files requires a base release version to be specified").ToString();
		case ELauncherProfileValidationErrors::GeneratingChunksRequiresCookByTheBook:
			return LOCTEXT("GeneratingChunksRequiresCookByTheBook", "Generating Chunks requires cooking or zen snapshot import.").ToString();
		case ELauncherProfileValidationErrors::GeneratingChunksRequiresUnrealPak:
			return LOCTEXT("GeneratingChunksRequiresUnrealPak", "UnrealPak must be selected to Generate Chunks.").ToString();
		case ELauncherProfileValidationErrors::GeneratingHttpChunkDataRequiresGeneratingChunks:
			return LOCTEXT("GeneratingHttpChunkDataRequiresGeneratingChunks", "Generate Chunks must be selected to Generate Http Chunk Install Data.").ToString();
		case ELauncherProfileValidationErrors::GeneratingHttpChunkDataRequiresValidDirectoryAndName:
			return LOCTEXT("GeneratingHttpChunkDataRequiresValidDirectoryAndName", "Generating Http Chunk Install Data requires a valid directory and release name.").ToString();
		case ELauncherProfileValidationErrors::ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly:
			return LOCTEXT("ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly", "Shipping doesn't support commandline options and can't use cook on the fly").ToString();
		case ELauncherProfileValidationErrors::CookOnTheFlyDoesntSupportServer:
			return LOCTEXT("CookOnTheFlyDoesntSupportServer", "Cook on the fly doesn't support server target configurations").ToString();
		case ELauncherProfileValidationErrors::LaunchDeviceIsUnauthorized:
			return LOCTEXT("LaunchDeviceIsUnauthorized", "Device is unauthorized or locked.").ToString();
		case ELauncherProfileValidationErrors::NoArchiveDirectorySpecified:
			return LOCTEXT("NoArchiveDirectorySpecifiedError", "The archive step requires a valid directory.").ToString();
		case ELauncherProfileValidationErrors::IoStoreRequiresPakFiles:
			return LOCTEXT("IoStoreRequiresPakFilesError", "UnrealPak must be selected when using I/O store container file(s)").ToString();
		case ELauncherProfileValidationErrors::BuildTargetCookVariantMismatch:
			return LOCTEXT("BuildTargetCookVariantMismatch", "Build Target and Platform Variant mismatch.").ToString();
		case ELauncherProfileValidationErrors::BuildTargetIsRequired:
			return LOCTEXT("BuildTargetIsRequired", "This profile requires an explicit Build Target set.").ToString();
		case ELauncherProfileValidationErrors::FallbackBuildTargetIsRequired:
			return LOCTEXT("FallbackBuildTargetIsRequired", "An explicit Default Build Target is required for the selected Variant.").ToString();
		case ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch:
			return LOCTEXT("ZenPakStreamingRequiresDeployAndLaunch", "Zen Pak Streaming requires deployment to a device and launching.").ToString();
		case ELauncherProfileValidationErrors::AutomatedTestRequiredBuildCookRun:
			return LOCTEXT("AutomatedTestRequiredBuildCookRun", "Automated Tests currently require at least one active BuildCookRun.").ToString();
		default:
			return TEXT("Unknown");
	};
}

TSharedPtr<UE::Zen::Build::FBuildListRetriever> FLauncherProfile::BuildListRetriever;
TMap<FString, FLauncherProfile::FPlatformToBuildsMap> FLauncherProfile::PerProjectBuilds;

void FLauncherProfile::QueryZenSnapshotsForProject()
{
	FString BuildType = TEXT("oplog");
	FString Branch = FEngineVersion::Current().GetBranch();
	Branch = Branch.Replace(TEXT("//"), TEXT("")).Replace(TEXT("/"), TEXT("-")).ToLower();

	TSharedPtr<UE::Zen::Build::FBuildListRetriever> BuildRetriever = FLauncherProfile::GetBuildListRetriever();
	BuildRetriever->QueryBuilds(GetProjectName(), BuildType, Branch,
		[this](const TMap<FString, TArray<int32>>& PerPlatformBuilds) mutable
		{
			check(IsInGameThread());

			FPlatformToBuildsMap& PlatformConfigToBuildMap = PerProjectBuilds.FindOrAdd(GetProjectName().ToLower());
			PlatformConfigToBuildMap.Reset();

			for (auto& Pair : PerPlatformBuilds)
			{
				// Find the closest changelist that is before the synced changelist
				int32 ClosestBuild = -1;
				const int32 ChangeListToMatch = FEngineVersion::Current().GetChangelist();
				for (int32 Build : Pair.Value)
				{
					if (Build <= ChangeListToMatch)
					{
						ClosestBuild = Build;
					}
					else
					{
						break;
					}
				}

				PlatformConfigToBuildMap.Add(Pair.Key.ToLower(), ClosestBuild);
			}
		}
	);
}

int32 FLauncherProfile::GetZenSnapshot()
{
	FString ProjectName = GetProjectName().ToLower();
	if (PerProjectBuilds.Contains(ProjectName))
	{
		TArray<FString> Platforms;
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : GetBuildCookRunCommands())
		{
			TArray<FString> ThisPlatforms = BuildCookRun->GetCookedPlatforms();
			for (const FString& Platform : ThisPlatforms)
			{
				Platforms.AddUnique(Platform);
			}
		}

		for (const FString& Platform : Platforms)
		{
			FString LowerPlatform = Platform.ToLower();
			FPlatformToBuildsMap& Builds = PerProjectBuilds[ProjectName];
			if (Builds.Contains(LowerPlatform))
			{
				return Builds[LowerPlatform];
			}
		}
	}
	else
	{
		QueryZenSnapshotsForProject();
	}

	return 0;
}

TSharedPtr<UE::Zen::Build::FBuildListRetriever> FLauncherProfile::GetBuildListRetriever()
{
	if (!BuildListRetriever)
	{
		BuildListRetriever = MakeShared<UE::Zen::Build::FBuildListRetriever>();
		BuildListRetriever->ConnectToBuildService();
	}
	return BuildListRetriever;
}




bool FLauncherProfile::Serialize( FArchive& Archive )
{
	// use our friend access to help with deprected serialization
	TSharedPtr<FLauncherProfileBuildCookRun> PrivateInner = StaticCastSharedPtr<FLauncherProfileBuildCookRun>(FirstBuildCookRun);
	if (!PrivateInner.IsValid())
	{
		return false;
	}

	int32 Version = LAUNCHERSERVICES_FINAL;

	Archive	<< Version;

	if (Version < LAUNCHERSERVICES_MINPROFILEVERSION)
	{
		return false;
	}

	if (Version > LAUNCHERSERVICES_FINAL)
	{
		return false;
	}

	if (Archive.IsSaving())
	{
		if (PrivateInner->DeployedDeviceGroup.IsValid())
		{
			PrivateInner->DeployedDeviceGroupId = PrivateInner->DeployedDeviceGroup->GetId();
		}
		else
		{
			PrivateInner->DeployedDeviceGroupId = FGuid();
		}
	}

	// IMPORTANT: make sure to bump LAUNCHERSERVICES_PROFILEVERSION when modifying this!
	bool BuildGame = false;
	Archive << Id
			<< Name
			<< Description
			<< PrivateInner->BuildConfiguration
			<< ProjectSpecified
			<< ShareableProjectPath
			<< PrivateInner->CookConfiguration
			<< PrivateInner->CookIncremental
			<< PrivateInner->CookOptions
			<< PrivateInner->CookMode
			<< PrivateInner->CookUnversioned
			<< PrivateInner->CookedCultures
			<< PrivateInner->CookedMaps
			<< PrivateInner->CookedPlatforms
			<< PrivateInner->DeployStreamingServer
			<< PrivateInner->DeployWithUnrealPak
			<< PrivateInner->DeployedDeviceGroupId
			<< PrivateInner->DeploymentMode
			<< PrivateInner->HideFileServerWindow
			<< PrivateInner->LaunchMode
			<< PrivateInner->PackagingMode
			<< PrivateInner->PackageDir
			<< BuildGame
            << PrivateInner->ForceClose
            << PrivateInner->Timeout;

	if (Version >= LAUNCHERSERVICES_SHAREABLEPROJECTPATHS)
	{
		FullProjectPath = FPaths::ConvertRelativePathToFull(FPaths::RootDir(), ShareableProjectPath);
	}

	FString DeployPlatformString = PrivateInner->DefaultDeployPlatform.ToString();
	if (Version >= LAUNCHERSERVICES_FIXCOMPRESSIONSERIALIZE)
	{
		Archive << PrivateInner->Compressed;
	}
	if ( Version>= LAUNCHERSERVICES_ADDEDENCRYPTINIFILES)
	{
		Archive << PrivateInner->EncryptIniFiles;
		Archive << PrivateInner->ForDistribution;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDDEFAULTDEPLOYPLATFORM)
	{
		Archive << DeployPlatformString;
	}		
	if (Version >= LAUNCHERSERVICES_ADDEDNUMCOOKERSTOSPAWN && Version < LAUNCHERSERVICES_REMOVEDNUMCOOKERSTOSPAWN)
	{
		int32 OldNumCookersToSpawn;
		Archive << OldNumCookersToSpawn;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDSKIPCOOKINGEDITORCONTENT)
	{
		Archive << PrivateInner->bSkipCookingEditorContent;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDINCREMENTALDEPLOYVERSION)
	{
		Archive << PrivateInner->DeployIncremental;
	}
	if ( Version >= LAUNCHERSERVICES_REMOVEDPATCHSOURCECONTENTPATH )
	{
		Archive << PrivateInner->GeneratePatch;
	}
	if ( Version >= LAUNCHERSERVICES_ADDEDMULTILEVELPATCHING )
	{
		Archive << PrivateInner->AddPatchLevel;
		Archive << StageBaseReleasePaks;
	}
	else if ( Version >= LAUNCHERSERVICES_ADDEDPATCHSOURCECONTENTPATH)
	{
		FString Temp;
		Archive << Temp;
		Archive << PrivateInner->GeneratePatch;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDDLCINCLUDEENGINECONTENT)
	{
		Archive << PrivateInner->DLCIncludeEngineContent;
	}
		
	if ( Version >= LAUNCHERSERVICES_ADDEDRELEASEVERSION )
	{
		Archive << PrivateInner->CreateReleaseVersion;
		Archive << PrivateInner->CreateReleaseVersionName;
		Archive << PrivateInner->BasedOnReleaseVersionName;
			
		Archive << PrivateInner->CreateDLC;
		Archive << PrivateInner->DLCName;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDGENERATECHUNKS)
	{
		Archive << PrivateInner->bGenerateChunks;
		Archive << PrivateInner->bGenerateHttpChunkData;
		Archive << PrivateInner->HttpChunkDataDirectory;
		Archive << PrivateInner->HttpChunkDataReleaseName;
	}
	if (Version >= LAUNCHERSERVICES_ADDARCHIVE)
	{
		Archive << PrivateInner->bArchive;
		Archive << PrivateInner->ArchiveDir;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDADDITIONALCOMMANDLINE)
	{
		Archive << PrivateInner->AdditionalCommandLineParameters;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDINCLUDEPREREQUISITES)
	{
		Archive << PrivateInner->IncludePrerequisites;
	}

	if (Version >= LAUNCHERSERVICES_ADDEDBUILDMODE)
	{
		Archive << PrivateInner->BuildMode;
	}
	else if(Archive.IsLoading())
	{
		PrivateInner->BuildMode = BuildGame ? ELauncherProfileBuildModes::Build : ELauncherProfileBuildModes::DoNotBuild;
	}

	if (Version >= LAUNCHERSERVICES_ADDEDUSEIOSTORE)
	{
		Archive << PrivateInner->bUseIoStore;
	}

	if (Version >= LAUNCHERSERVICES_ADDEDMAKEBINARYCONFIG)
	{
		Archive << PrivateInner->bMakeBinaryConfig;
	}

	if (Version >= LAUNCHERSERVICES_ADDEDREFERENCECONTAINERS)
	{
		Archive << PrivateInner->ReferenceContainerGlobalFileName;
		Archive << PrivateInner->ReferenceContainerCryptoKeysFileName;
	}
	if (Version >= LAUNCHERSERVICES_ADDEDORIGINALRELEASEVERSION)
	{
		Archive << PrivateInner->OriginalReleaseVersionName;
	}

	FString BuildTargetName;
	if (Version >= LAUNCHERSERVICES_ADDBUILDTARGETNAME)
	{
		Archive << PrivateInner->BuildTargetSpecified;
		Archive << BuildTargetName;
	}

	if (Version >= LAUNCHERSERVICES_ADDEDRETAINSTAGEDDIRECTORY && Version < LAUNCHERSERVICES_REMOVEDRETAINSTAGEDDIRECTORY)
	{
		bool bRetainStagedDirectory = false;
		Archive << bRetainStagedDirectory;
	}

	if (Version >= LAUNCHERSERVICES_ADDZENOPTIONS)
	{
		Archive << PrivateInner->bUseZenStore;
		Archive << PrivateInner->bImportingZenSnapshot;
		Archive << PrivateInner->bUseZenPakStreaming;
		Archive << PrivateInner->ZenPakStreamingPath;
	}

	if (Version >= LAUNCHERSERVICES_ADDBUILDARCHITECTURE)
	{
		Archive << PrivateInner->ClientArchitectures;
		Archive << PrivateInner->ServerArchitectures;
		Archive << PrivateInner->EditorArchitectures;
	}

	if (Version >= LAUNCHERSERVICES_ADDCUSTOMPROPERTIES)
	{
		Archive << CustomStringProperties;
		Archive << CustomBoolProperties;
	}

	if (Version >= LAUNCHERSERVICES_INCREMENTALCOOKMODE && Version < LAUNCHERSERVICES_REMOVEDINCREMENTALCOOKMODE)
	{
		TEnumAsByte<ELauncherProfileIncrementalCookMode::Type> IncrementalCookMode = (PrivateInner->CookIncremental ? ELauncherProfileIncrementalCookMode::ModifiedOnly : ELauncherProfileIncrementalCookMode::None);
		Archive << IncrementalCookMode;
	}

	if (Version >= LAUNCHERSERVICES_AUTOMATEDTESTS)
	{
		TArray<TSharedPtr<ILauncherProfileAutomatedTest>> AutomatedTests;
		if (Archive.IsSaving())
		{
			for (const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
			{
				ILauncherProfileAutomatedTestPtr AutomatedTest = UATCommand->AsAutomatedTest();
				if (AutomatedTest)
				{
					AutomatedTests.Add(AutomatedTest);
				}
			}
		}

		// serialize automated tests
		int NumAutomatedTests = AutomatedTests.Num();
		Archive << NumAutomatedTests;
		for( int32 TestIndex = 0; TestIndex < NumAutomatedTests; TestIndex++)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Archive.IsLoading())
			{
				ILauncherProfileAutomatedTestRef AutomatedTest = CreateAutomatedTestInternal(TEXT(""), TEXT(""));
				AutomatedTest->Serialize(Archive);
				AutomatedTests.Add(AutomatedTest);
			}
			else
			{
				AutomatedTests[TestIndex]->Serialize(Archive);
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (Archive.IsLoading())
		{
			for (TSharedPtr<ILauncherProfileAutomatedTest> AutomatedTest : AutomatedTests)
			{
				UATCommands.Add(AutomatedTest.ToSharedRef());
			}
		}

		Archive << AutomatedTestBuildPath;
		Archive << bWantAutomatedTestBuild;
	}

	if (Version >= LAUNCHERSERVICES_ADDMULTIPLEBUILDTARGETS)
	{
		Archive << PrivateInner->BuildTargets;
		if (Archive.IsLoading() && PrivateInner->BuildTargets.Num() == 0 && !BuildTargetName.IsEmpty())
		{
			PrivateInner->BuildTargets.Add(BuildTargetName);
		}

		Archive << PrivateInner->AdditionalTargetCommandLineParameters;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PrivateInner->DefaultLaunchRole->Serialize(Archive);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// serialize launch roles
	if (Archive.IsLoading())
	{
		PrivateInner->DeployedDeviceGroup.Reset();
		PrivateInner->LaunchRoles.Reset();
	}

	int32 NumLaunchRoles = PrivateInner->LaunchRoles.Num();

	Archive << NumLaunchRoles;

	for (int32 RoleIndex = 0; RoleIndex < NumLaunchRoles; ++RoleIndex)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Archive.IsLoading())
		{
			PrivateInner->LaunchRoles.Add(MakeShareable(new FLauncherProfileLaunchRole(Archive)));				
		}
		else
		{
			PrivateInner->LaunchRoles[RoleIndex]->Serialize(Archive);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (Archive.IsLoading())
	{
		PrivateInner->DefaultDeployPlatform = FName(*DeployPlatformString);
	}

	if (PrivateInner->DefaultDeployPlatform != NAME_None)
	{
		PrivateInner->SetDefaultDeployPlatform(PrivateInner->DefaultDeployPlatform);
	}

	if (Archive.IsLoading())
	{
		RefreshValidBuildTargets();
	}
	Validate();

	return true;
}





#undef LOCTEXT_NAMESPACE

