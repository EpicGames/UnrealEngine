// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "ILauncherProfile.h"
#include "ILauncherServicesModule.h"
#include "Misc/Paths.h"
#include "Launcher/LauncherProjectPath.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Culture.h"
#include "Misc/App.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceServicesModule.h"
#include "GameProjectHelper.h"
#include "Profiles/LauncherProfileVersion.h"
#include "Profiles/LauncherProfileLaunchRole.h"
#include "PlatformInfo.h"
#include "TargetReceipt.h"
#include "DesktopPlatformModule.h"
#include "GameProjectHelper.h"
#include "Experimental/BuildListRetriever.h"

class Error;

enum ESimpleLauncherVersion
{
	LAUNCHERSERVICES_SIMPLEPROFILEVERSION=1,
	LAUNCHERSERVICES_SIMPLEFILEFORMATCHANGE = 2,
};

/**
* Implements a simple profile which controls the desired output of the Launcher for simple
*/
class FLauncherSimpleProfile final
	: public ILauncherSimpleProfile
{
public:

	FLauncherSimpleProfile(const FString& InDeviceName)
		: DeviceName(InDeviceName)
	{
		SetDefaults();
	}

	//~ Begin ILauncherSimpleProfile Interface

	virtual const FString& GetDeviceName() const override
	{
		return DeviceName;
	}

	virtual FName GetDeviceVariant() const  override
	{
		return Variant;
	}

	virtual EBuildConfiguration GetBuildConfiguration() const override
	{
		return BuildConfiguration;
	}

	virtual ELauncherProfileCookModes::Type GetCookMode() const override
	{
		return CookMode;
	}

	virtual void SetDeviceName(const FString& InDeviceName) override
	{
		if (DeviceName != InDeviceName)
		{
			DeviceName = InDeviceName;
		}
	}

	virtual void SetDeviceVariant(FName InVariant) override
	{
		Variant = InVariant;
	}

	virtual void SetBuildConfiguration(EBuildConfiguration InConfiguration) override
	{
		BuildConfiguration = InConfiguration;
	}

	virtual void SetCookMode(ELauncherProfileCookModes::Type InMode) override
	{
		CookMode = InMode;
	}

	UE_DEPRECATED(5.8, "this will be removed. there is no alternative implementation : ILauncherSimpleProfile is designed to be transient")
	virtual bool Serialize(FArchive& Archive) override
	{
		int32 Version = LAUNCHERSERVICES_SIMPLEPROFILEVERSION;

		Archive << Version;

		if (Version != LAUNCHERSERVICES_SIMPLEPROFILEVERSION)
		{
			return false;
		}

		// IMPORTANT: make sure to bump LAUNCHERSERVICES_SIMPLEPROFILEVERSION when modifying this!
		Archive << DeviceName
			<< Variant
			<< BuildConfiguration
			<< CookMode;

		return true;
	}

	virtual void Save(TJsonWriter<>& Writer) override
	{
		int32 Version = LAUNCHERSERVICES_SIMPLEFILEFORMATCHANGE;

		Writer.WriteObjectStart();
		Writer.WriteValue(TEXT("Version"), Version);
		Writer.WriteValue(TEXT("DeviceName"), DeviceName);
		Writer.WriteValue(TEXT("Variant"), Variant.ToString());
		Writer.WriteValue(TEXT("BuildConfiguration"), (int32)BuildConfiguration);
		Writer.WriteValue(TEXT("CookMode"), CookMode);
		Writer.WriteObjectEnd();
	}

	virtual bool Load(const FJsonObject& Object) override
	{
		int32 Version = (int32)Object.GetNumberField(TEXT("Version"));
		if (Version < LAUNCHERSERVICES_SIMPLEFILEFORMATCHANGE)
		{
			return false;
		}

		DeviceName = Object.GetStringField(TEXT("DeviceName"));
		Variant = *(Object.GetStringField(TEXT("Variant")));
		BuildConfiguration = (EBuildConfiguration)((int32)Object.GetNumberField(TEXT("BuildConfiguration")));
		CookMode = (TEnumAsByte<ELauncherProfileCookModes::Type>)((int32)Object.GetNumberField(TEXT("CookMode")));

		return true;
	}

	virtual void SetDefaults() override
	{
		// None will mean the preferred Variant for the device is used.
		Variant = NAME_None;
		
		// I don't use FApp::GetBuildConfiguration() because i don't want the act of running in debug the first time to cause the simple
		// profiles created for your persistent devices to be in debug. The user might not see this if they don't expand the Advanced options.
		BuildConfiguration = EBuildConfiguration::Development;
		
		CookMode = ELauncherProfileCookModes::OnTheFly;
	}

private:

	// Holds the name of the device this simple profile is for
	FString DeviceName;

	// Holds the name of the device variant.
	FName Variant;

	// Holds the desired build configuration (only used if creating new builds).
	EBuildConfiguration BuildConfiguration;

	// Holds the cooking mode.
	TEnumAsByte<ELauncherProfileCookModes::Type> CookMode;
};


/**
 * Implements a profile which controls the desired output of the Launcher
 */
class FLauncherProfile final
	: public ILauncherProfile
	, public TSharedFromThis<FLauncherProfile>
{
public:

	/**
	* Gets the folder in which profile files are stored.
	*
	* @return The folder path.
	*/
	static FString GetProfileFolder(bool bNotForLicensees)
	{
		if (bNotForLicensees)
		{
			return FPaths::EngineDir() / TEXT("Restricted/NotForLicensees/Programs/UnrealFrontend/Profiles");
		}
		return FPaths::EngineDir() / TEXT("Programs/UnrealFrontend/Profiles");
	}

	/**
	 * Default constructor.
	 */
	FLauncherProfile(ILauncherProfileManagerRef ProfileManager)
		: LauncherProfileManager(ProfileManager)
	{ 
		ProjectChangedDelegate.AddRaw(this, &FLauncherProfile::OnSelectedProjectChanged);
	}

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InProfileName - The name of the profile.
	 */
	FLauncherProfile(ILauncherProfileManagerRef ProfileManager, FGuid InId, const FString& InProfileName)
		: LauncherProfileManager(ProfileManager)
		, Id(InId)
		, Name(InProfileName)
	{
		ProjectChangedDelegate.AddRaw(this, &FLauncherProfile::OnSelectedProjectChanged);
	}

	/**
	 * Destructor.
	 */
	~FLauncherProfile( ) 
	{
	}


	struct FValidationData
	{
		void Reset()
		{
			Errors.Reset();
			CustomErrors.Reset();
			CustomWarnings.Reset();
			InvalidPlatform.Reset();
		}

		// Holds the collection of validation errors.
		TArray<ELauncherProfileValidationErrors::Type> Errors;
		FString InvalidPlatform;

		// Holds the collection of custom errors & warnings.
		TMap<FString,FText> CustomErrors;
		TMap<FString,FText> CustomWarnings;
	};



	//~ Begin ILauncherProfile Interface

	virtual void AddCookedCulture( const FString& CultureName ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->AddCookedCulture(CultureName);
	}

	virtual void AddCookedMap( const FString& MapName ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->AddCookedMap(MapName);
	}

	virtual void AddCookedPlatform( const FString& PlatformName ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->AddCookedPlatform(PlatformName);
		RefreshValidBuildTargets();
		Validate();
	}

	virtual void SetDefaultDeployPlatform(const FName PlatformName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetDefaultDeployPlatform(PlatformName);
	}

	virtual void ClearCookedCultures( ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->ClearCookedCultures();
	}

	virtual void ClearCookedMaps( ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->ClearCookedMaps();
	}

	virtual void ClearCookedPlatforms( ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->ClearCookedPlatforms();
		RefreshValidBuildTargets();
		Validate();
	}

	virtual ILauncherProfileLaunchRolePtr CreateLaunchRole( ) override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->CreateLaunchRole();
	}

	virtual EBuildConfiguration GetBuildConfiguration( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetBuildConfiguration();;
	}

	virtual bool HasBuildTargetSpecified() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->HasBuildTargetSpecified();
	}

	virtual FString GetBuildTarget() const override
	{
		if (HasBuildTargetSpecified())
		{
			check(FirstBuildCookRun.IsValid());
			TArray<FString> BuildTargets = FirstBuildCookRun->GetBuildTargets();
			UE_CLOGF(BuildTargets.Num() > 0, LogTemp, Warning, "GetBuildTargets is only returning the first build target - legacy project launcher?");

			return (BuildTargets.Num() > 0) ? BuildTargets[0] : FString();
		}
		else
		{
			return LauncherProfileManager->GetBuildTarget();
		}
	}

	virtual EBuildConfiguration GetCookConfiguration( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCookConfiguration();
	}

	virtual ELauncherProfileCookModes::Type GetCookMode( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCookMode();
	}

	virtual const FString& GetCookOptions( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCookOptions();
	}

	virtual const TArray<FString>& GetCookedCultures( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCookedCultures();
	}

	virtual const bool GetSkipCookingEditorContent() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetSkipCookingEditorContent();
	}

	virtual const TArray<FString>& GetCookedMaps( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCookedMaps();
	}

	virtual const TArray<FString>& GetCookedPlatforms( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCookedPlatforms();
	}

	virtual const ILauncherProfileLaunchRoleRef& GetDefaultLaunchRole( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetDefaultLaunchRole();
	}

	virtual ILauncherDeviceGroupPtr GetDeployedDeviceGroup( ) override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetDeployedDeviceGroup();
	}

	virtual const FName GetDefaultDeployPlatform() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetDefaultDeployPlatform();
	}

	virtual bool IsGeneratingPatch() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsGeneratingPatch();
	}

	virtual bool ShouldAddPatchLevel() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->ShouldAddPatchLevel();
	}

	virtual bool ShouldStageBaseReleasePaks() const override
	{
		return StageBaseReleasePaks;
	}

	virtual bool IsCreatingDLC() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsCreatingDLC();
	}
	virtual void SetCreateDLC(bool InBuildDLC) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetCreateDLC(InBuildDLC);
	}

	virtual FString GetDLCName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetDLCName();
	}
	virtual void SetDLCName(const FString& InDLCName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetDLCName(InDLCName);
	}

	virtual bool IsDLCIncludingEngineContent() const
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsDLCIncludingEngineContent();
	}
	virtual void SetDLCIncludeEngineContent(bool InDLCIncludeEngineContent)
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetDLCIncludeEngineContent(InDLCIncludeEngineContent);
	}



	virtual bool IsCreatingReleaseVersion() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsCreatingReleaseVersion();
	}

	virtual void SetCreateReleaseVersion(bool InCreateReleaseVersion) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetCreateReleaseVersion(InCreateReleaseVersion);
	}

	virtual FString GetCreateReleaseVersionName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetCreateReleaseVersionName();
	}

	virtual void SetCreateReleaseVersionName(const FString& InCreateReleaseVersionName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetCreateReleaseVersionName(InCreateReleaseVersionName);
	}


	virtual FString GetBasedOnReleaseVersionName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetBasedOnReleaseVersionName();
	}

	virtual void SetBasedOnReleaseVersionName(const FString& InBasedOnReleaseVersionName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetBasedOnReleaseVersionName(InBasedOnReleaseVersionName);
	}

	virtual FString GetOriginalReleaseVersionName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetOriginalReleaseVersionName(); 
	}

	virtual void SetOriginalReleaseVersionName(const FString& InOriginalReleaseVersionName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetOriginalReleaseVersionName(InOriginalReleaseVersionName);
	}

	virtual FString GetReferenceContainerGlobalFileName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetReferenceContainerGlobalFileName();
	}
	virtual void SetReferenceContainerGlobalFileName(const FString& InReferenceContainerGlobalFileName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetReferenceContainerGlobalFileName(InReferenceContainerGlobalFileName);
	}
	virtual FString GetReferenceContainerCryptoKeysFileName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetReferenceContainerCryptoKeysFileName();
	}
	virtual void SetReferenceContainerCryptoKeysFileName(const FString& InReferenceContainerCryptoKeysFileName) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->SetReferenceContainerCryptoKeysFileName(InReferenceContainerCryptoKeysFileName);
	}

	virtual ELauncherProfileDeploymentModes::Type GetDeploymentMode( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetDeploymentMode();
	}

    virtual bool GetForceClose() const override
    {
		check(FirstBuildCookRun.IsValid());
        return FirstBuildCookRun->GetForceClose();
    }
    
	virtual FGuid GetId( ) const override
	{
		return Id;
	}

	virtual void AssignId( bool bOverrideExisting ) override
	{
		if (bOverrideExisting || !Id.IsValid())
		{
			Id = FGuid::NewGuid();
		}
	}


	virtual FString GetFileName() const override
	{
		//toupper for filename so that filepaths can be compared the same on case sensitive and case-insensitive platforms
		return GetName().ToUpper() + TEXT("_") + GetId().ToString() + TEXT(".ulp2");
	}

	virtual FString GetFilePath() const override
	{
		return GetProfileFolder(bNotForLicensees) / GetFileName();
	}

	virtual ELauncherProfileLaunchModes::Type GetLaunchMode( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetLaunchMode();
	}

	virtual const TArray<ILauncherProfileLaunchRolePtr>& GetLaunchRoles( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetLaunchRoles();
	}

	virtual const int32 GetLaunchRolesFor( const FString& DeviceId, TArray<ILauncherProfileLaunchRolePtr>& OutRoles ) override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetLaunchRolesFor( DeviceId, OutRoles );
	}

	virtual FString GetName( ) const override
	{
		return Name;
	}

	virtual FString GetDescription() const override
	{
		return Description;
	}

	virtual ELauncherProfilePackagingModes::Type GetPackagingMode( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetPackagingMode();
	}

	virtual FString GetPackageDirectory( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetPackageDirectory();
	}

	virtual bool IsArchiving( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsArchiving();
	}

	virtual FString GetArchiveDirectory( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetArchiveDirectory();
	}

	virtual bool HasProjectSpecified() const override
	{
		return ProjectSpecified;
	}

	virtual FString GetProjectName() const override
	{
		if (ProjectSpecified)
		{
			FString Path = FLauncherProjectPath::GetProjectName(FullProjectPath);
			return Path;
		}
		return LauncherProfileManager->GetProjectName();
	}

	virtual FString GetProjectBasePath() const override
	{
		if (ProjectSpecified)
		{
			return FLauncherProjectPath::GetProjectBasePath(FullProjectPath);
		}
		return LauncherProfileManager->GetProjectBasePath();
	}

	virtual FString GetProjectPath() const override
	{
		if (ProjectSpecified)
		{
			return FullProjectPath;
		}
		return LauncherProfileManager->GetProjectPath();
	}

    virtual uint32 GetTimeout() const override
    {
		check(FirstBuildCookRun.IsValid());
        return FirstBuildCookRun->GetTimeout();
    }

	virtual bool HasValidationError(ILauncherProfileUATCommandPtr UATCommand) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		return ValidationData && ValidationData->Errors.Num() > 0;
	}
    
	virtual bool HasValidationError( ELauncherProfileValidationErrors::Type Error, ILauncherProfileUATCommandPtr UATCommand ) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		return ValidationData && ValidationData->Errors.Contains(Error);
	}

	virtual void AddCustomError( const FString& UniqueId, const FText& Text, ILauncherProfileUATCommandPtr UATCommand ) override
	{
		FValidationData* ValidationData = GetValidationData(UATCommand);
		if (ValidationData)
		{
			ValidationData->CustomErrors.Add(UniqueId, Text);
		}
	}

	virtual bool HasCustomError( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand ) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		return ValidationData && ValidationData->CustomErrors.Contains(UniqueId);
	}

	virtual FText GetCustomErrorText( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand ) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		const FText* TextPtr = ValidationData ? ValidationData->CustomErrors.Find(UniqueId) : nullptr;
		return TextPtr ? (*TextPtr) : FText::GetEmpty();
	}

	virtual TArray<FString> GetAllCustomErrors(ILauncherProfileUATCommandPtr UATCommand) const override
	{
		TArray<FString> Keys;

		const FValidationData* ValidationData = GetValidationData(UATCommand);
		if (ValidationData)
		{
			ValidationData->CustomErrors.GetKeys(Keys);
		}

		return MoveTemp(Keys);
	}

	virtual void AddCustomWarning( const FString& UniqueId, const FText& Text, ILauncherProfileUATCommandPtr UATCommand ) override
	{
		FValidationData* ValidationData = GetValidationData(UATCommand);
		if (ValidationData)
		{
			ValidationData->CustomWarnings.Add(UniqueId, Text);
		}
	}

	virtual bool HasCustomWarning( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand ) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		return ValidationData && ValidationData->CustomWarnings.Contains(UniqueId);
	}

	virtual FText GetCustomWarningText( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand ) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		const FText* TextPtr = ValidationData ? ValidationData->CustomWarnings.Find(UniqueId) : nullptr;
		return TextPtr ? (*TextPtr) : FText::GetEmpty();
	}

	virtual TArray<FString> GetAllCustomWarnings(ILauncherProfileUATCommandPtr UATCommand) const override
	{
		TArray<FString> Keys;
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		if (ValidationData)
		{
			ValidationData->CustomWarnings.GetKeys(Keys);
		}
		return MoveTemp(Keys);
	}

	virtual void RefreshCustomWarningsAndErrors(ILauncherProfileUATCommandPtr UATCommand) override
	{
		FValidationData* ValidationData = GetValidationData(UATCommand);
		if (ValidationData)
		{
			ValidationData->CustomWarnings.Reset();
			ValidationData->CustomErrors.Reset();
		}

		if (UATCommand == nullptr)
		{
			CustomValidationDelegate.Broadcast();
		}
		else
		{
			CustomUATCommandValidationDelegate.Broadcast(UATCommand.ToSharedRef());
		}
	}



	virtual FString GetInvalidPlatform(ILauncherProfileUATCommandPtr UATCommand) const override
	{
		const FValidationData* ValidationData = GetValidationData(UATCommand);
		return ValidationData ? ValidationData->InvalidPlatform : FString();
	}

	virtual ELauncherProfileBuildModes::Type GetBuildMode() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetBuildMode();
	}

	virtual bool ShouldBuild() override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->ShouldBuild();
	}

	virtual bool IsFastIterate() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsFastIterate();
	}
	
	virtual bool IsBuildingUAT() const override
	{
		return BuildUAT;
	}

	virtual FString GetAdditionalCommandLineParameters() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetAdditionalCommandLineParameters();
	}

	virtual FString GetAdditionalTargetCommandLineParameters( EBuildTargetType BuildTargetType ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetAdditionalTargetCommandLineParameters(BuildTargetType);
	}

	virtual bool IsCookingIncrementally( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsCookingIncrementally();
	}

	virtual ELauncherProfileIncrementalCookMode::Type GetIncrementalCookMode() const override
	{
		// fixme: worth reading LegacyIterative to honor this value?!
		return IsCookingIncrementally() ? ELauncherProfileIncrementalCookMode::ModifiedAndDependencies : ELauncherProfileIncrementalCookMode::None;
	}


	virtual bool IsIterateSharedCookedBuild() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsIterateSharedCookedBuild();
	}

	virtual bool IsCompressed() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsCompressed();
	}

	virtual bool IsEncryptingIniFiles() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsEncryptingIniFiles();
	}

	virtual bool IsForDistribution() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsForDistribution();
	}

	virtual bool IsCookingUnversioned( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsCookingUnversioned();
	}

	virtual bool IsDeployablePlatform( const FString& PlatformName ) override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsDeployablePlatform(PlatformName);
	}

	virtual bool IsDeployingIncrementally( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsDeployingIncrementally();
	}

	virtual bool IsFileServerHidden( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsFileServerHidden();
	}

	virtual bool IsFileServerStreaming( ) const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsFileServerStreaming();
	}

	virtual bool IsPackingWithUnrealPak( ) const  override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsPackingWithUnrealPak();
	}

	virtual bool IsIncludingPrerequisites() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsIncludingPrerequisites();
	}

	virtual bool IsGeneratingChunks() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsGeneratingChunks();
	}

	virtual bool IsGenerateHttpChunkData() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->IsGenerateHttpChunkData();
	}

	virtual FString GetHttpChunkDataDirectory() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetHttpChunkDataDirectory();
	}

	virtual FString GetHttpChunkDataReleaseName() const override
	{
		check(FirstBuildCookRun.IsValid());
		return FirstBuildCookRun->GetHttpChunkDataReleaseName();
	}

	virtual bool IsValidForLaunch( ) override
	{
		if (ProfileValidation.Errors.Num() != 0 || ProfileValidation.CustomErrors.Num() != 0)
		{
			return false;
		}

		for (const TPair<ILauncherProfileUATCommandPtr,FValidationData>& Pair : UATCommandValidation)
		{
			if (Pair.Value.Errors.Num() != 0 || Pair.Value.CustomErrors.Num() != 0)
			{
				return false;
			}
		}

		return true;
	}

	virtual void RemoveCookedCulture( const FString& CultureName ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->RemoveCookedCulture(CultureName);

		Validate();
	}

	virtual void RemoveCookedMap( const FString& MapName ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->RemoveCookedMap(MapName);

		Validate();
	}

	virtual void RemoveCookedPlatform( const FString& PlatformName ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->RemoveCookedPlatform(PlatformName);

		RefreshValidBuildTargets();
		Validate();
	}

	virtual void RemoveLaunchRole( const ILauncherProfileLaunchRoleRef& Role ) override
	{
		check(FirstBuildCookRun.IsValid());
		FirstBuildCookRun->RemoveLaunchRole(Role);

		Validate();
	}

	UE_DEPRECATED(5.8, "please use the Json Save/Load API instead")
	virtual bool Serialize( FArchive& Archive ) override;

	virtual void Save(TJsonWriter<>& Writer) override
	{
		int32 Version = LAUNCHERSERVICES_FINAL;

		Writer.WriteObjectStart();
		Writer.WriteValue("Version", Version);
		Writer.WriteValue("Id", Id.ToString());
		Writer.WriteValue("Name", Name);
		Writer.WriteValue("Description", Description);
		Writer.WriteValue("ProjectSpecified", ProjectSpecified);
		Writer.WriteValue("ShareableProjectPath", ShareableProjectPath);
		Writer.WriteValue("StageBaseReleasePaks", StageBaseReleasePaks); // deprecated in 5.6
		Writer.WriteValue("CustomStringProperties", CustomStringProperties);
		Writer.WriteValue("CustomBoolProperties", CustomBoolProperties);

		// serialize the automated test info
		Writer.WriteValue("AutomatedTestBuildPath", AutomatedTestBuildPath);
		Writer.WriteValue("WantAutomatedTestBuild", bWantAutomatedTestBuild);

		// serialize the UAT commands
		Writer.WriteArrayStart("UATCommands");
		for ( const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
		{
			UATCommand->Save(Writer);
		}
		Writer.WriteArrayEnd();

		// write out the UAT project params
		if (FirstBuildCookRun.IsValid())
		{
			SaveUATParams(Writer);
		}
		Writer.WriteObjectEnd();
	}

	void SaveUATParams(TJsonWriter<>& Writer)
	{
		Writer.WriteArrayStart("scripts");
		Writer.WriteObjectStart();

		TArray<FString> Platforms = FindPlatforms();

		// script to run
		Writer.WriteValue("script", TEXT("BuildCookRun"));

		// project to operate on
		Writer.WriteValue("project", ProjectSpecified ? ShareableProjectPath : "");

		// ancillary arguments
		Writer.WriteValue("noP4", true);
		Writer.WriteValue("nocompile", !IsBuildingUAT());
		Writer.WriteValue("nocompileeditor", FApp::IsEngineInstalled());
		Writer.WriteValue("unrealexe", GetEditorExe());
		Writer.WriteValue("utf8output", true);

		// client configurations
		Writer.WriteArrayStart("clientconfig");
		Writer.WriteValue(LexToString(GetBuildConfiguration()));
		Writer.WriteArrayEnd();

		// server configurations
		Writer.WriteArrayStart("serverconfig");
		Writer.WriteValue(LexToString(GetBuildConfiguration()));
		Writer.WriteArrayEnd();

		// platforms
		TArray<FString> ServerPlatforms;
		TArray<FString> ClientPlatforms;
		FString OptionalParams;
		bool ClosesAfterLaunch = FindAllPlatforms(ServerPlatforms, ClientPlatforms, OptionalParams);
		if (ServerPlatforms.Num() > 0)
		{
			Writer.WriteValue("server", true);
			Writer.WriteArrayStart("serverplatform");
			for (int32 Idx = 0; Idx < ServerPlatforms.Num(); Idx++)
			{
				Writer.WriteValue(ServerPlatforms[Idx]);
			}
			Writer.WriteArrayEnd();
		}

		if (ClientPlatforms.Num() > 0)
		{
			Writer.WriteArrayStart("platform");
			for (int32 Idx = 0; Idx < ClientPlatforms.Num(); Idx++)
			{
				Writer.WriteValue(ClientPlatforms[Idx]);
			}
			Writer.WriteArrayEnd();
		}

		// optional params
		TMap<FString, FString> OptionalCommands = ParseCommands(OptionalParams);
		for (TMap<FString, FString>::TIterator Iter = OptionalCommands.CreateIterator(); Iter; ++Iter)
		{
			Writer.WriteValue(Iter.Key(), Iter.Value());
		}

		// game command line
		FString InitialMap = GetDefaultLaunchRole()->GetInitialMap();
		if (InitialMap.IsEmpty() && GetCookedMaps().Num() == 1)
		{
			InitialMap = GetCookedMaps()[0];
		}
		Writer.WriteObjectStart("cmdline");
		Writer.WriteValue("", InitialMap);
		Writer.WriteValue("messaging", true);
		Writer.WriteObjectEnd();

		// devices
		ITargetDeviceServicesModule& DeviceServiceModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>(TEXT("TargetDeviceServices"));
		TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager = DeviceServiceModule.GetDeviceProxyManager();
		ILauncherDeviceGroupPtr DeviceGroup = GetDeployedDeviceGroup();
		TMap<FString, FString> RoleCommands;
		FString CommandLine = GetDefaultLaunchRole()->GetUATCommandLine();
		if (CommandLine.Len() > 0)
		{
			// parse out the commands
			TMap<FString, FString> Commands = ParseCommands(CommandLine);
			RoleCommands.Append(Commands);
		}
		if (DeviceGroup.IsValid())
		{
			const TArray<FString>& Devices = DeviceGroup->GetDeviceIDs();
			bool bUseVsync = false;

			Writer.WriteArrayStart("device");

			// for each deployed device...
			for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
			{
				const FString& DeviceId = Devices[DeviceIndex];
				TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceId);
				Writer.WriteValue(DeviceId);
				if (DeviceProxy.IsValid())
				{
					TArray<ILauncherProfileLaunchRolePtr> Roles;
					if (GetLaunchRolesFor(DeviceId, Roles) > 0)
					{
						for (int32 RoleIndex = 0; RoleIndex < Roles.Num(); RoleIndex++)
						{
							if (!bUseVsync && Roles[RoleIndex]->IsVsyncEnabled())
							{
								bUseVsync = true;
							}
							CommandLine = Roles[RoleIndex]->GetUATCommandLine();
							TMap<FString, FString> Commands = ParseCommands(CommandLine);
							RoleCommands.Append(Commands);
						}
					}
				}
			}
			Writer.WriteArrayEnd();
		}

		// write out the additional command-line arguments
		static FGuid SessionId(FGuid::NewGuid());
		Writer.WriteObjectStart("addcmdline");
		Writer.WriteValue("sessionid", SessionId.ToString());
		Writer.WriteValue("sessionowner", FPlatformProcess::UserName(true));
		Writer.WriteValue("sessionname", GetName());
		for (TMap<FString, FString>::TIterator Iter = RoleCommands.CreateIterator(); Iter; ++Iter)
		{
			Writer.WriteValue(Iter.Key(), Iter.Value());
		}
		Writer.WriteObjectEnd();

		// map list
		Writer.WriteArrayStart("map");
		const TArray<FString>& CookedMapsArray = GetCookedMaps();
		if (CookedMapsArray.Num() > 0 && (GetCookMode() == ELauncherProfileCookModes::ByTheBook || GetCookMode() == ELauncherProfileCookModes::ByTheBookInEditor))
		{
			for (int32 MapIndex = 0; MapIndex < CookedMapsArray.Num(); ++MapIndex)
			{
				Writer.WriteValue(CookedMapsArray[MapIndex]);
			}
		}
		else
		{
			Writer.WriteValue(InitialMap);
		}
		Writer.WriteArrayEnd();

		// staging directory
		auto PackageDirectory = GetPackageDirectory();
		if (PackageDirectory.Len() > 0)
		{
			Writer.WriteValue("stagingdirectory", PackageDirectory);
		}

		// build
		Writer.WriteValue("build", ShouldBuild());

		// cook
		switch (GetCookMode())
		{
		case ELauncherProfileCookModes::ByTheBook:
			{
				Writer.WriteValue("cook", true);
				Writer.WriteValue("unversionedcookedcontent", IsCookingUnversioned());
				Writer.WriteValue("pak", IsPackingWithUnrealPak());

				if (IsCreatingReleaseVersion())
				{
					Writer.WriteValue("createreleaseversion", GetCreateReleaseVersionName());
				}

				if (IsCreatingDLC())
				{
					Writer.WriteValue("dlcname", GetDLCName());
				}

				Writer.WriteValue("generatepatch", IsGeneratingPatch());

				if (IsGeneratingPatch() || IsCreatingReleaseVersion() || IsCreatingDLC())
				{
					if (GetBasedOnReleaseVersionName().IsEmpty() == false)
					{
						Writer.WriteValue("basedonreleaseversion", GetBasedOnReleaseVersionName());
						Writer.WriteValue("stagebasereleasepaks", ShouldStageBaseReleasePaks());
					}

					if (GetOriginalReleaseVersionName().IsEmpty() == false)
					{
						Writer.WriteValue("originalreleaseversion", GetOriginalReleaseVersionName());
					}
				}

				if (IsGeneratingPatch())
				{
					Writer.WriteValue("addpatchlevel", ShouldAddPatchLevel());
				}

				Writer.WriteValue("manifests", IsGeneratingChunks());

				if (IsGenerateHttpChunkData())
				{
					Writer.WriteValue("createchunkinstall", true);
					Writer.WriteValue("chunkinstalldirectory", GetHttpChunkDataDirectory());
					Writer.WriteValue("chunkinstallversion", GetHttpChunkDataReleaseName());
				}

				if (IsArchiving())
				{
					Writer.WriteValue("archive", true);
					Writer.WriteValue("archivedirectory", GetArchiveDirectory());
				}

				TMap<FString, FString> CookCommands = ParseCommands(GetCookOptions());
				for (TMap<FString, FString>::TIterator Iter = CookCommands.CreateIterator(); Iter; ++Iter)
				{
					Writer.WriteValue(Iter.Key(), Iter.Value());
				}
			}
			break;
		case ELauncherProfileCookModes::OnTheFly:
			{
				Writer.WriteValue("cookonthefly", true);

				//if UAT doesn't stick around as long as the process we are going to run, then we can't kill the COTF server when UAT goes down because the program
				//will still need it.  If UAT DOES stick around with the process then we DO want the COTF server to die with UAT so the next time we launch we don't end up
				//with two COTF servers.
				if (ClosesAfterLaunch)
				{
					Writer.WriteValue("nokill", true);
				}
			}
			break;
		case ELauncherProfileCookModes::OnTheFlyInEditor:
			Writer.WriteValue("skipcook", true);
			Writer.WriteValue("cookonthefly", true);
			break;
		case ELauncherProfileCookModes::ByTheBookInEditor:
			Writer.WriteValue("skipcook", true);
			break;
		case ELauncherProfileCookModes::DoNotCook:
			Writer.WriteValue("skipcook", true);
			break;
		}

		Writer.WriteValue("iterativecooking", IsCookingIncrementally());
		Writer.WriteValue("iteratesharedcookedbuild", IsIterateSharedCookedBuild() );
		Writer.WriteValue("skipcookingeditorcontent", GetSkipCookingEditorContent());
		Writer.WriteValue("compressed", IsCompressed());
		Writer.WriteValue("EncryptIniFiles", IsEncryptingIniFiles());
		Writer.WriteValue("ForDistribution", IsForDistribution());

		// stage/package/deploy
		bool bIsStaging = false;
		if (GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy)
		{
			switch (GetDeploymentMode())
			{
			case ELauncherProfileDeploymentModes::CopyRepository:
				{
					Writer.WriteValue("skipstage", true);
					Writer.WriteValue("deploy", true);
				}
				break;

			case ELauncherProfileDeploymentModes::CopyToDevice:
				{
					Writer.WriteValue("iterativedeploy", IsDeployingIncrementally());
				}
			case ELauncherProfileDeploymentModes::FileServer:
				{
					Writer.WriteValue("stage", true);
					bIsStaging = true;
					Writer.WriteValue("deploy", true);
				}
				break;
			}

			// run
			if (GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch)
			{
				Writer.WriteValue("run", true);
			}
		}
		else
		{
			if (GetPackagingMode() == ELauncherProfilePackagingModes::Locally)
			{
				Writer.WriteValue("stage", true);
				bIsStaging = true;
				Writer.WriteValue("package", true);
			}
		}

		if (bIsStaging &&
			GetReferenceContainerGlobalFileName().Len())
		{
			// (only iostore uses this) - pass reference chunk database
			Writer.WriteValue("ReferenceContainerGlobalFileName", GetReferenceContainerGlobalFileName());
			if (GetReferenceContainerCryptoKeysFileName().Len())
			{
				Writer.WriteValue("ReferenceContainerCryptoKeys", GetReferenceContainerCryptoKeysFileName());
			}
		}

		if (GetClientArchitectures().Num() > 0)
		{
			Writer.WriteValue("clientarchitecture", FString::Join(GetClientArchitectures(), TEXT("+")));
		}
		if (GetServerArchitectures().Num() > 0)
		{
			Writer.WriteValue("serverarchitecture", FString::Join(GetServerArchitectures(), TEXT("+")));
		}
		if (GetEditorArchitectures().Num() > 0)
		{
			Writer.WriteValue("editorarchitecture", FString::Join(GetEditorArchitectures(), TEXT("+")));
		}

		/*
		"script", ""
		"project", ""
		"i18npreset", ""
		"cookcultures", ["", "", ""]
		"targetplatform, ["". "", ""]
		"servertargetplatform, ["", "", ""]
		"build", "true/false"
		"run", "true/false"
		"cook, "true/false"
		"cookflavor, ""
		"createreleaseversionroot", ""
		"basedonreleaseversionroot", ""
		"createreleaseversion", ""
		"baseonreleaseversion", ""
		"generatepatch", "true/false"
		"additionalcookeroptions", ["","",""]
		"dlcname", ""
		"diffcookedcontentpath", ""
		"dlcincludeengine", "true/false"
		"skipcook", "true/false"
		"clean", "true/false"
		"signpak", ""
		"signedpak", "true/false"
		"pak", "true/false"
		"skippak", "true/false"
		"noxge", "true/false"
		"cookonthefly", "true/false"
		"cookontheflystreaming", "true/false"
		"unversionedcookcontent", "true/false"
		"skipcookingeditorcontent", "true/false"
		"compressed", "true/false"
		"iterativecooking", "true/false"
		"skipcookonthefly", "true/false"
		"cookall", "true/false"
		"cookmapsonly", "true/false"
		"fileserver", "true/false"
		"dedicatedserver", "true/false"
		"client", "true/false"
		"noclient", "true/false"
		"logwindow", "true/false"
		"stage", "true/false"
		"skipstage", "true/false"
		"stagingdirectory", ""
		"stagenonmonolithic", "true/false"
		"codesign", "true/false"
		"manifests", "true/false"
		"createchunkinstall", "true/false"
		"chunkinstalldirectory", ""
		"chunkinstallversion", ""
		"archive", "true/false"
		"archivedirectory", ""
		"archivemetadata", "true/false"
		"distrbution", "true/false"
		"prereqs", "true/false"
		"nobootstrapexe", "true/false"
		"prebuilt", "true/false"
		"nodebuginfo", "true/false"
		"nocleanstage", "true/false"
		"maptorun", ""
		"additionalservermapparams", ["", "", ""]
		"foreign", "true/false"
		"foreigncode", "true/false"
		"cmdline", ["", "", ""]
		"bundlename", ""
		"addcmdline", ["", "", ""]
		"package", "true/false"
		"deploy", "true/false"
		"iterativedeploy", "true/false"
		"fastcook", "true/false"
		"ignorecookerrors", "true/false"
		"uploadsymbols", "true/false"
		"device", ""
		"serverdevice", ""
		"nullrhi", "true/false"
		"fakeclient", "true/false"
		"editortests", "true/false"
		"runautomationtest", ""
		"runautomationtests", "true/false"
		"skipserver", "true/false"
		"unrealexe", ""
		"unattended", "true/false"
		"deviceuser", ""
		"devicepass", ""
		"crashreporter", "true/false"
		"specifiedarchitecture", ""
		"clientconfig", ["", "", ""]
		"serverconfig", ["", "", ""]
		"port", ["", "", ""]
		"mapstocook", ["", "", ""]
		"numclients", "8"
		"crashindex", "8"
		"runtimeoutseconds", "8"
		*/
		Writer.WriteObjectEnd();
		Writer.WriteArrayEnd();
	}

	TMap<FString, FString> ParseCommands(FString CommandLine)
	{
		TMap<FString, FString> RoleCommands;
		FString Left;
		FString Right;
		while (CommandLine.Split(TEXT(" "), &Left, &Right))
		{
			if(Left.Len() > 0)
			{
				FString Key;
				FString Value;
				if (!Left.Split(TEXT("="), &Key, &Value))
				{
					Key = Left;
					Value = TEXT("true");
				}
				Key.RemoveFromStart(TEXT("-"));
				RoleCommands.Add(Key, Value);
			}
			CommandLine = Right;
		}
		if (CommandLine.Len() > 0)
		{
			if (!CommandLine.Split(TEXT("="), &Left, &Right))
			{
				Right = TEXT("true");
			}
			Left.RemoveFromStart(TEXT("-"));
			RoleCommands.Add(Left, Right);
		}

		return RoleCommands;
	}

	TArray<FString> FindPlatforms()
	{
		TArray<FString> Platforms;
		if (GetCookMode() == ELauncherProfileCookModes::ByTheBook)
		{
			Platforms = GetCookedPlatforms();
		}

		// determine deployment platforms
		ILauncherDeviceGroupPtr DeviceGroup = GetDeployedDeviceGroup();
		FName Variant = NAME_None;

		// Loading the Device Proxy Manager to get the needed Device Manager.
		ITargetDeviceServicesModule* DeviceServiceModule = FModuleManager::LoadModulePtr<ITargetDeviceServicesModule>(TEXT("TargetDeviceServices"));
		if (DeviceServiceModule == nullptr)
		{
			return Platforms;
		}
		TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager = DeviceServiceModule->GetDeviceProxyManager();

		if (DeviceGroup.IsValid() && Platforms.Num() < 1)
		{
			const TArray<FString>& Devices = DeviceGroup->GetDeviceIDs();
			// for each deployed device...
			for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
			{
				const FString& DeviceId = Devices[DeviceIndex];

				TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceId);

				if (DeviceProxy.IsValid())
				{
					// add the platform
					Variant = DeviceProxy->GetTargetDeviceVariant(DeviceId);
					Platforms.AddUnique(DeviceProxy->GetTargetPlatformName(Variant));
				}
			}
		}

		return Platforms;
	}

	bool FindAllPlatforms(TArray<FString>& ServerPlatforms, TArray<FString>& ClientPlatforms, FString& OptionalParams)
	{
		bool bUATClosesAfterLaunch = false;
		TArray<FString> InPlatforms = FindPlatforms();
		for (int32 PlatformIndex = 0; PlatformIndex < InPlatforms.Num(); ++PlatformIndex)
		{
			// Platform info for the given platform
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(*InPlatforms[PlatformIndex]));
			if (PlatformInfo == nullptr)
			{
				return false;
			}

			// add the UBT platform name to the appropriate list
			TArray<FString>& Platforms = PlatformInfo->PlatformType == EBuildTargetType::Server ? ServerPlatforms : ClientPlatforms;
			Platforms.Add(PlatformInfo->DataDrivenPlatformInfo->UBTPlatformName.ToString());

			// Append any extra UAT flags specified for this platform flavor
			if (!PlatformInfo->UATCommandLine.IsEmpty())
			{
				OptionalParams += PlatformInfo->UATCommandLine;
			}

			bUATClosesAfterLaunch |= PlatformInfo->DataDrivenPlatformInfo->bUATClosesAfterLaunch;
		}

		// If both Client and Server are desired to be built avoid Server causing clients to not be built PlatformInfo wise
		if (OptionalParams.Contains(TEXT("-client")) && OptionalParams.Contains(TEXT("-noclient")))
		{
			OptionalParams = OptionalParams.Replace(TEXT("-noclient"), TEXT(""));
		}

		return bUATClosesAfterLaunch;
	}

	virtual bool Load(const FJsonObject& Object) override
	{
		int32 Version = (int32)Object.GetNumberField(TEXT("Version"));
		if (Version < LAUNCHERSERVICES_FILEFORMATCHANGE || Version > LAUNCHERSERVICES_FINAL)
		{
			return false;
		}
		UATCommands.Reset();
		UATCommandValidation.Reset();
		FirstBuildCookRun.Reset();
		
		if (Version < LAUNCHERSERVICES_INNERBUILDCOOKRUN)
		{
			ILauncherProfileBuildCookRunPtr NewInner = LoadBuildCookRunInternal(Object, Version);
			if (!NewInner.IsValid())
			{
				return false;
			}

			FirstBuildCookRun = NewInner;
			AddUATCommandInternal(NewInner.ToSharedRef());
		}		

		FGuid::Parse(Object.GetStringField(TEXT("Id")), Id);
		Name = Object.GetStringField(TEXT("Name"));
		Description = Object.GetStringField(TEXT("Description"));
		ProjectSpecified = Object.GetBoolField(TEXT("ProjectSpecified"));
		ShareableProjectPath = Object.GetStringField(TEXT("ShareableProjectPath"));

		if (Version >= LAUNCHERSERVICES_ADDCUSTOMPROPERTIES)
		{
			const TSharedPtr<FJsonObject>* CustomPropertiesObject;

			CustomStringProperties.Reset();
			if (Object.TryGetObjectField(TEXT("CustomStringProperties"), CustomPropertiesObject))
			{
				for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KeyPairs : (*CustomPropertiesObject)->Values)
				{
					CustomStringProperties.Add(FString(KeyPairs.Key), KeyPairs.Value->AsString());
				}
			}

			CustomBoolProperties.Reset();
			if (Object.TryGetObjectField(TEXT("CustomBoolProperties"), CustomPropertiesObject))
			{
				for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KeyPairs : (*CustomPropertiesObject)->Values)
				{
					CustomBoolProperties.Add(FString(KeyPairs.Key), KeyPairs.Value->AsBool());
				}
			}
		}

		if (Version >= LAUNCHERSERVICES_AUTOMATEDTESTS)
		{
			if (Version < LAUNCHERSERVICES_ADDUATCOMMANDS)
			{
				const TArray<TSharedPtr<FJsonValue>>* Values = nullptr;
				if (Object.TryGetArrayField(TEXT("AutomatedTests"), Values))
				{
					for (TSharedPtr<FJsonValue> Value : *Values)
					{
						ILauncherProfileUATCommandRef UATCommand = CreateAutomatedTestInternal(TEXT(""), TEXT(""));
						if (UATCommand->Load((*Value->AsObject().Get())))
						{
							AddUATCommandInternal(UATCommand);
						}
						else
						{
							UE_LOGF(LogLauncherProfile, Error, "cannot parse automated test");
							return false;
						}
					}
				}
			}

			AutomatedTestBuildPath = Object.GetStringField(TEXT("AutomatedTestBuildPath"));
			bWantAutomatedTestBuild = Object.GetBoolField(TEXT("WantAutomatedTestBuild"));
		}


		if (Version >= LAUNCHERSERVICES_ADDUATCOMMANDS)
		{
			const TArray<TSharedPtr<FJsonValue>>* Values = nullptr;
			if (Object.TryGetArrayField(TEXT("UATCommands"), Values))
			{
				for (TSharedPtr<FJsonValue> Value : *Values)
				{
					const FJsonObject& JsonObject = *Value->AsObject().Get();
					FString UATCommandType = JsonObject.GetStringField(TEXT("Type"));

					ILauncherProfileUATCommandPtr UATCommand = CreateUATCommandByTypeNameInternal(*UATCommandType, TEXT(""));
					if (!UATCommand.IsValid())
					{
						UE_LOGF(LogLauncherProfile, Error, "unknown UAT command type: %ls", *UATCommandType);
						return false;
					}
					else if (UATCommand->Load(JsonObject))
					{
						AddUATCommandInternal(UATCommand.ToSharedRef());
					}
					else
					{
						UE_LOGF(LogLauncherProfile, Error, "cannot parse UAT command type: %ls", *UATCommandType);
						return false;
					}
				}
			}

			if (Version >= LAUNCHERSERVICES_INNERBUILDCOOKRUN)
			{
				TArray<ILauncherProfileBuildCookRunRef> BuildCookRunCommands = GetBuildCookRunCommands();
				FirstBuildCookRun = (BuildCookRunCommands.Num() > 0) ? BuildCookRunCommands[0].ToSharedPtr() : nullptr;
			}
		}

		// if (Version >= LAUNCHERSERVICES_SHAREABLEPROJECTPATHS) Always true due to early out at top of function
		{
			FullProjectPath = FPaths::ConvertRelativePathToFull(FPaths::RootDir(), ShareableProjectPath);
		}

		RefreshValidBuildTargets();
		Validate();

		return true;
	}

	virtual void SetDefaults( ) override
	{
		ProjectSpecified = false;

		// default project settings
		if (FPaths::IsProjectFilePathSet())
		{
			FullProjectPath = FPaths::GetProjectFilePath();
		}
		else if (FGameProjectHelper::IsGameAvailable(FApp::GetProjectName()))
		{
			FullProjectPath = FPaths::RootDir() / FApp::GetProjectName() / FApp::GetProjectName() + TEXT(".uproject");
		}
		else
		{
			FullProjectPath = FString();
		}

		FString RelativeProjectPath = FullProjectPath;
		bool bRelative = FPaths::MakePathRelativeTo(RelativeProjectPath, *FPaths::RootDir());

		bool bIsUnderUERoot = bRelative && !(RelativeProjectPath.StartsWith(FString("../"), ESearchCase::CaseSensitive));
		if (bIsUnderUERoot)
		{
			ShareableProjectPath = RelativeProjectPath;
		}
		else
		{
			ShareableProjectPath = FullProjectPath;
		}

		// Use the locally specified project path is resolving through the root isn't working
		ProjectSpecified = !FullProjectPath.IsEmpty();
		
		// default build settings
		BuildUAT = !FApp::GetEngineIsPromotedBuild() && !FApp::IsEngineInstalled();


		// default deploy settings
		StageBaseReleasePaks = false; // deprecated in 5.6
		// default UAT settings
		ILauncherServicesModule& LauncherServicesModule = FModuleManager::GetModuleChecked<ILauncherServicesModule>(TEXT("LauncherServices"));
		EditorExe = LauncherServicesModule.GetExecutableForCommandlets();

		bNotForLicensees = false;
		bShouldUpdateFlash = false;

		AutomatedTestBuildPath.Reset();
		bWantAutomatedTestBuild = false;

		UATCommands.Reset();
		UATCommandValidation.Reset();
		FirstBuildCookRun = CreateBuildCookRunInternal(TEXT("FirstBuildCookRun"), TEXT(""));
		AddUATCommandInternal(FirstBuildCookRun.ToSharedRef());

		RefreshValidBuildTargets();
		Validate();
	}

	virtual void SetBuildMode(ELauncherProfileBuildModes::Type Mode) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetBuildMode(Mode);
	}

	virtual void SetFastIterate(bool Enable) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetFastIterate(Enable);
	}
	
	virtual void SetBuildUAT(bool Build) override
	{
		if (BuildUAT != Build)
		{
			BuildUAT = Build;

			Validate();
		}
	}

	virtual void SetAdditionalCommandLineParameters(const FString& Params) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetAdditionalCommandLineParameters(Params);
	}

	virtual void SetAdditionalTargetCommandLineParameters(const FString& Params, EBuildTargetType BuildTargetType ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetAdditionalTargetCommandLineParameters(Params, BuildTargetType);
	}



	virtual void SetBuildConfiguration( EBuildConfiguration Configuration ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetBuildConfiguration(Configuration);
	}

	virtual void SetBuildTargetSpecified(bool Specified) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetBuildTargetSpecified(Specified);
	}

	virtual void FallbackBuildTargetUpdated() override
	{
		if (!HasBuildTargetSpecified())
		{
			Validate();
		}
	}


	virtual void SetBuildTarget( const FString& TargetName ) override
	{
		check(FirstBuildCookRun.IsValid())
		TArray<FString> BuildTargets = FirstBuildCookRun->GetBuildTargets();

		FString BuildTargetName = (BuildTargets.Num() > 0) ? BuildTargets[0] : FString();
		if (BuildTargetName != TargetName || BuildTargets.Num() > 1)
		{
			UE_CLOGF(BuildTargets.Num() > 0, LogTemp, Warning, "SetBuildTarget removed multiple targets - legacy project launcher?");

			FirstBuildCookRun->ClearBuildTargets();
			FirstBuildCookRun->AddBuildTarget(TargetName);

			Validate();
		}
	}

	virtual void SetCookConfiguration( EBuildConfiguration Configuration ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetCookConfiguration(Configuration);
	}

	virtual void SetCookMode( ELauncherProfileCookModes::Type Mode ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetCookMode(Mode);
	}

	virtual void SetCookOptions(const FString& Options) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetCookOptions(Options);
	}

	virtual void SetSkipCookingEditorContent(const bool InSkipCookingEditorContent) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetSkipCookingEditorContent(InSkipCookingEditorContent);
	}

	virtual void SetDeployWithUnrealPak( bool UseUnrealPak ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetDeployWithUnrealPak(UseUnrealPak);
	}

	virtual void SetGenerateChunks(bool bInGenerateChunks) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetGenerateChunks(bInGenerateChunks);
	}

	virtual void SetGenerateHttpChunkData(bool bInGenerateHttpChunkData) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetGenerateHttpChunkData(bInGenerateHttpChunkData);
	}

	virtual void SetHttpChunkDataDirectory(const FString& InHttpChunkDataDirectory) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetHttpChunkDataDirectory(InHttpChunkDataDirectory);
	}

	virtual void SetHttpChunkDataReleaseName(const FString& InHttpChunkDataReleaseName) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetHttpChunkDataReleaseName(InHttpChunkDataReleaseName);
	}

	virtual void SetDeployedDeviceGroup( const ILauncherDeviceGroupPtr& DeviceGroup ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetDeployedDeviceGroup(DeviceGroup);
	}

	virtual FIsCookFinishedDelegate& OnIsCookFinished() override
	{
		return IsCookFinishedDelegate;
	}

	virtual FCookCanceledDelegate& OnCookCanceled() override
	{
		return CookCanceledDelegate;
	}

	virtual void SetDeploymentMode( ELauncherProfileDeploymentModes::Type Mode ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetDeploymentMode(Mode);
	}

    virtual void SetForceClose( bool Close ) override
    {
		check(FirstBuildCookRun.IsValid())
        FirstBuildCookRun->SetForceClose(Close);
    }
    
	virtual void SetHideFileServerWindow( bool Hide ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetHideFileServerWindow(Hide);
	}

	virtual void SetIncrementalCooking( bool Incremental ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetIncrementalCooking(Incremental);
	}

	virtual void SetIncrementalCookMode( ELauncherProfileIncrementalCookMode::Type Mode ) override
	{
		check(FirstBuildCookRun.IsValid())
		bool Incremental = (Mode != ELauncherProfileIncrementalCookMode::None);
		FirstBuildCookRun->SetIncrementalCooking(Incremental);
	}


	virtual void SetIterateSharedCookedBuild( bool SharedCookedBuild ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetIterateSharedCookedBuild(SharedCookedBuild);
	}

	virtual void SetCompressed( bool Enabled ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetCompressed(Enabled);
	}

	virtual void SetForDistribution(bool Enabled)override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetForDistribution(Enabled);
	}

	virtual void SetEncryptingIniFiles(bool Enabled) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetEncryptingIniFiles(Enabled);
	}

	virtual void SetIncrementalDeploying( bool Incremental ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetIncrementalDeploying(Incremental);
	}

	virtual void SetLaunchMode( ELauncherProfileLaunchModes::Type Mode ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetLaunchMode(Mode);
	}

	virtual void SetName( const FString& NewName ) override
	{
		if (Name != NewName)
		{
			Name = NewName;

			Validate();
		}
	}

	virtual void SetDescription(const FString& NewDescription) override
	{
		if (Description != NewDescription)
		{
			Description = NewDescription;

			Validate();
		}
	}

	virtual void SetNotForLicensees() override
	{
		bNotForLicensees = true;
	}

	virtual void SetPackagingMode( ELauncherProfilePackagingModes::Type Mode ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetPackagingMode(Mode);
	}

	virtual void SetPackageDirectory( const FString& Dir ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetPackageDirectory(Dir);
	}

	virtual void SetArchive( bool bInArchive ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetArchive(bInArchive);
	}

	virtual void SetArchiveDirectory( const FString& Dir ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetArchiveDirectory(Dir);
	}

	virtual void SetProjectSpecified(bool Specified) override
	{
		if (ProjectSpecified != Specified)
		{
			ProjectSpecified = Specified;

			RefreshValidBuildTargets();
			Validate();

			ProjectChangedDelegate.Broadcast();
		}
	}

	virtual void FallbackProjectUpdated() override
	{
		if (!HasProjectSpecified())
		{
			RefreshValidBuildTargets();
			Validate();

			ProjectChangedDelegate.Broadcast();
		}
	}

	virtual void SetProjectPath( const FString& Path ) override
	{
		if (FullProjectPath != Path)
		{
			if(Path.IsEmpty())
			{
				FullProjectPath = Path;
			}
			else
			{
				FullProjectPath = FPaths::ConvertRelativePathToFull(Path);

				FString RelativeProjectPath = Path;
				bool bRelative = FPaths::MakePathRelativeTo(RelativeProjectPath, *FPaths::RootDir());

				bool bIsUnderUERoot = bRelative && !(RelativeProjectPath.StartsWith(FString("../"), ESearchCase::CaseSensitive));
				if (bIsUnderUERoot)
				{
					ShareableProjectPath = RelativeProjectPath;
				}
				else
				{
					ShareableProjectPath = FullProjectPath;
				}				
			}

			for (const ILauncherProfileBuildCookRunRef& BuildCookRun : GetBuildCookRunCommands())
			{
				BuildCookRun->ClearCookedMaps();
			}

			RefreshValidBuildTargets();
			Validate();

			ProjectChangedDelegate.Broadcast();
		}
	}

	virtual void SetStreamingFileServer( bool Streaming ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetStreamingFileServer(Streaming);
	}

	virtual void SetIncludePrerequisites(bool InValue) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetIncludePrerequisites(InValue);
	}

    virtual void SetTimeout( uint32 InTime ) override
    {
		check(FirstBuildCookRun.IsValid())
        FirstBuildCookRun->SetTimeout(InTime);
    }
    
	virtual void SetUnversionedCooking( bool Unversioned ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetUnversionedCooking(Unversioned);
	}

	virtual void SetGeneratePatch( bool InGeneratePatch ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetGeneratePatch(InGeneratePatch);
	}

	virtual void SetAddPatchLevel( bool InAddPatchLevel) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetAddPatchLevel(InAddPatchLevel);
	}

	virtual void SetStageBaseReleasePaks(bool InStageBaseReleasePaks) override
	{
		StageBaseReleasePaks = InStageBaseReleasePaks;
	}

	virtual bool SupportsEngineMaps( ) const override
	{
		return false;
	}

	virtual FOnProfileProjectChanged& OnProjectChanged() override
	{
		return ProjectChangedDelegate;
	}

	virtual FOnProfileBuildTargetOptionsChanged& OnBuildTargetOptionsChanged() override
	{
		return BuildTargetOptionsChangedDelegate;
	}

	virtual FOnProfileCustomValidation& OnCustomValidation() override
	{
		return CustomValidationDelegate;
	}

	virtual FOnProfileCustomUATCommandValidation& OnCustomUATCommandValidation() override
	{
		return CustomUATCommandValidationDelegate;
	}

	virtual void SetEditorExe( const FString& InEditorExe ) override
	{
		EditorExe = InEditorExe;
	}

	virtual FString GetEditorExe( ) const override
	{
		return EditorExe;
	}

	virtual void SetUseIoStore(bool bInUseIoStore) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetUseIoStore(bInUseIoStore);
	}

	virtual bool IsUsingIoStore() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->IsUsingIoStore();
	}

	virtual void SetUseZenStore(bool bInUseZenStore) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetUseZenStore(bInUseZenStore);
	}

	virtual bool IsUsingZenStore() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->IsUsingZenStore();
	}

	virtual int32 GetZenSnapshot() override;

	virtual void SetImportingZenSnapshot( bool Import ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetImportingZenSnapshot(Import);
	}

	virtual bool IsImportingZenSnapshot() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->IsImportingZenSnapshot();
	}

	virtual void SetUseZenPakStreaming( bool UseZenPakStreaming ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetUseZenPakStreaming(UseZenPakStreaming);
	}

	virtual bool IsUsingZenPakStreaming() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->IsUsingZenPakStreaming();
	}

	virtual void SetZenPakStreamingPath( const FString& Path ) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetZenPakStreamingPath(Path);
	}

	virtual FString GetZenPakStreamingPath() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->GetZenPakStreamingPath();
	}

	virtual void SetShouldUpdateDeviceFlash(bool bInShouldUpdateFlash) override
	{
		bShouldUpdateFlash = bInShouldUpdateFlash;
	}

	/**
	 * Whether or not the flash image/software on the device should attempt to be updated before running
	 */
	virtual bool ShouldUpdateDeviceFlash() const override
	{
		return bShouldUpdateFlash;
	}

	virtual void SetDeviceIsASimulator(bool bInIsDeviceASimualtor) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetDeviceIsASimulator(bInIsDeviceASimualtor);
	}

	/**
	 * Is the Launch device actually a simulator?
	 */
	virtual bool IsDeviceASimulator() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->IsDeviceASimulator();
	}

	virtual void SetMakeBinaryConfig(bool bInMakeBinaryConfig) override
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetMakeBinaryConfig(bInMakeBinaryConfig);
	}

	virtual bool MakeBinaryConfig() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->MakeBinaryConfig();
	}

	virtual TArray<FString> GetExplicitBuildTargetNames() const override
	{
		return ExplictBuildTargetNames;
	}

	virtual bool RequiresExplicitBuildTargetName() const override
	{
		return ExplictBuildTargetNames.Num() > 0;
	}

	virtual void SetClientArchitectures( const TArray<FString>& InArchitectures )
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetClientArchitectures(InArchitectures);
	}

	virtual const TArray<FString>& GetClientArchitectures() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->GetClientArchitectures();
	}

	virtual void SetServerArchitectures( const TArray<FString>& InArchitectures )
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetServerArchitectures(InArchitectures);
	}

	virtual const TArray<FString>& GetServerArchitectures() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->GetServerArchitectures();
	}

	virtual void SetEditorArchitectures( const TArray<FString>& InArchitectures )
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->SetEditorArchitectures(InArchitectures);
	}

	virtual const TArray<FString>& GetEditorArchitectures() const override
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->GetEditorArchitectures();
	}

	virtual TMap<FString,FString>& GetCustomStringProperties() override
	{
		return CustomStringProperties;
	}

	virtual TMap<FString, bool>& GetCustomBoolProperties() override
	{
		return CustomBoolProperties;
	}

	virtual const TArray<ILauncherProfileAutomatedTestRef> GetAutomatedTests() const override
	{
		TArray<ILauncherProfileAutomatedTestRef> AutomatedTests;
		for (const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
		{
			ILauncherProfileAutomatedTestPtr AutomatedTest = UATCommand->AsAutomatedTest();
			if (AutomatedTest)
			{
				AutomatedTests.Add(AutomatedTest.ToSharedRef());
			}
		}

		return MoveTemp(AutomatedTests);
	}

	virtual ILauncherProfileAutomatedTestPtr GetAutomatedTest( const TCHAR* InternalName ) const override
	{
		ILauncherProfileUATCommandPtr UATCommand = GetUATCommand(InternalName);
		return UATCommand ? UATCommand->AsAutomatedTest() : nullptr;
	}

	virtual ILauncherProfileAutomatedTestRef FindOrAddAutomatedTest( const TCHAR* InternalName, const TCHAR* UserTypeName ) override
	{
		// if no name has been given, autogenerate one
		FString UniqueName;
		if (InternalName == nullptr)
		{
			UniqueName = GetUniqueUATCommandName( UserTypeName ? UserTypeName : TEXT("CustomAutomatedTest"));
			InternalName = UniqueName.GetCharArray().GetData();
		}

		ILauncherProfileAutomatedTestPtr AutomatedTest = GetAutomatedTest(InternalName);
		if (AutomatedTest == nullptr)
		{
			AutomatedTest = CreateAutomatedTestInternal(InternalName, UserTypeName);
			AddUATCommandInternal(AutomatedTest.ToSharedRef());
			UATCommandAdded.Broadcast(AutomatedTest.ToSharedRef());
		}

		return AutomatedTest.ToSharedRef();
	}

	virtual void RemoveAutomatedTest( const TCHAR* InternalName ) override
	{
		RemoveUATCommand(InternalName);
	}

	virtual void SetAutomatedTestBuildPath( const FString& InAutomatedTestBuildPath )
	{
		AutomatedTestBuildPath = InAutomatedTestBuildPath;
	}
	virtual const FString& GetAutomatedTestBuildPath() const
	{
		return AutomatedTestBuildPath;
	}

	virtual void SetIsUsingAutomatedTestBuild( bool bInWantAutomatedTestBuild ) override
	{
		bWantAutomatedTestBuild = bInWantAutomatedTestBuild;
	}
	virtual bool IsUsingAutomatedTestBuild() const override
	{
		return bWantAutomatedTestBuild;
	}





	virtual const TArray<ILauncherProfileBuildCookRunRef> GetBuildCookRunCommands() const override
	{
		TArray<ILauncherProfileBuildCookRunRef> BuildCookRuns;
		for (const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
		{
			ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun();
			if (BuildCookRun)
			{
				BuildCookRuns.Add(BuildCookRun.ToSharedRef());
			}
		}

		return MoveTemp(BuildCookRuns);

	}

	virtual ILauncherProfileBuildCookRunPtr GetBuildCookRunCommand( const TCHAR* InternalName ) const override
	{
		ILauncherProfileUATCommandPtr UATCommand = GetUATCommand(InternalName);
		return UATCommand ? UATCommand->AsBuildCookRun() : nullptr;
	}

	virtual ILauncherProfileBuildCookRunRef FindOrAddBuildCookRunCommand( const TCHAR* InternalName, const TCHAR* UserTypeName ) override
	{
		// if no name has been given, autogenerate one
		FString UniqueName;
		if (InternalName == nullptr)
		{
			UniqueName = GetUniqueUATCommandName(UserTypeName ? UserTypeName : TEXT("CustomBuildCookRun"));
			InternalName = UniqueName.GetCharArray().GetData();
		}


		ILauncherProfileBuildCookRunPtr BuildCookRun = GetBuildCookRunCommand(InternalName);
		if (BuildCookRun == nullptr)
		{
			BuildCookRun = CreateBuildCookRunInternal(InternalName, UserTypeName);
			BuildCookRun->SetDeployedDeviceGroup(LauncherProfileManager->AddNewDeviceGroup());

			if (!FirstBuildCookRun.IsValid())
			{
				FirstBuildCookRun = BuildCookRun;
			}

			AddUATCommandInternal(BuildCookRun.ToSharedRef());
			UATCommandAdded.Broadcast(BuildCookRun.ToSharedRef());
		}

		return BuildCookRun.ToSharedRef();
	}



	virtual const TArray<ILauncherProfileUATCommandRef>& GetUATCommands() const override
	{
		return UATCommands;
	}

	virtual ILauncherProfileUATCommandPtr GetUATCommand( const TCHAR* InternalName ) const
	{
		const ILauncherProfileUATCommandRef* CommandPtr = UATCommands.FindByPredicate([InternalName]( const ILauncherProfileUATCommandRef& Test )
		{
			return FCString::Stricmp( InternalName, Test->GetInternalName()) == 0;
		});

		return CommandPtr ? CommandPtr->ToSharedPtr() : nullptr;

	}

	virtual ILauncherProfileUATCommandRef FindOrAddUATCommand( const TCHAR* InternalName, const TCHAR* UserTypeName ) override
	{
		// if no name has been given, autogenerate one
		FString UniqueName;
		if (InternalName == nullptr)
		{
			UniqueName = GetUniqueUATCommandName( UserTypeName ? UserTypeName : TEXT("CustomUATCommand"));
			InternalName = UniqueName.GetCharArray().GetData();
		}

		ILauncherProfileUATCommandPtr UATCommand = GetUATCommand(InternalName);
		if (UATCommand == nullptr)
		{
			UATCommand = CreateUATCommandInternal(InternalName, UserTypeName);

			AddUATCommandInternal(UATCommand.ToSharedRef());
			UATCommandAdded.Broadcast(UATCommand.ToSharedRef());

		}

		return UATCommand.ToSharedRef();
	}

	virtual void RemoveUATCommand( const TCHAR* InternalName ) override
	{
		ILauncherProfileUATCommandPtr UATCommand = GetUATCommand(InternalName);
		if (UATCommand != nullptr)
		{
			UATCommandRemoved.Broadcast(UATCommand.ToSharedRef());
			UATCommandValidation.Remove(UATCommand);
			UATCommands.Remove(UATCommand.ToSharedRef());

			// re-cache the first build cook run command if necessary
			if (UATCommand == FirstBuildCookRun)
			{
				TArray<ILauncherProfileBuildCookRunRef> BuildCookRunCommands = GetBuildCookRunCommands();
				FirstBuildCookRun = (BuildCookRunCommands.Num() > 0) ? BuildCookRunCommands[0].ToSharedPtr() : nullptr;
			}
		}
	}

	virtual FOnLauncherProfileUATCommandChanged& OnUATCommandAdded() override
	{
		return UATCommandAdded;
	}

	virtual FOnLauncherProfileUATCommandChanged& OnUATCommandRemoved() override
	{
		return UATCommandRemoved;
	}






	virtual TArray<FString> GetBuildTargets() const
	{
		check(FirstBuildCookRun.IsValid())
		return FirstBuildCookRun->GetBuildTargets();
	}

	virtual void AddBuildTarget( const FString& InBuildTarget )
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->AddBuildTarget(InBuildTarget);
		Validate();
	}

	virtual void RemoveBuildTarget( const FString& InBuildTarget )
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->RemoveBuildTarget(InBuildTarget);
		Validate();
	}

	virtual void ClearBuildTargets()
	{
		check(FirstBuildCookRun.IsValid())
		FirstBuildCookRun->ClearBuildTargets();
		Validate();
	}


	virtual bool SupportsLegacyProjectLauncher() const override
	{
		return FirstBuildCookRun.IsValid() && UATCommands.Num() == 1;
	}

	virtual ILauncherProfileBuildCookRunPtr GetFirstBuildCookRun() const override
	{
		return FirstBuildCookRun;
	}


	//~ End ILauncherProfile Interface

protected:

	/**
	 * Validates the profile's current settings.
	 *
	 * Possible validation errors and warnings can be retrieved using the HasValidationError() method.
	 *
	 * @return true if the profile passed validation, false otherwise.
	 */
	void Validate( )
	{
		ProfileValidation.Reset();
		RefreshCustomWarningsAndErrors(nullptr);

		TArray<ILauncherProfileBuildCookRunRef> BuildCookRuns = GetBuildCookRunCommands();

		// Build: a project must be selected
		if (GetProjectPath().IsEmpty())
		{
			ProfileValidation.Errors.Add(ELauncherProfileValidationErrors::NoProjectSelected);
		}

		// Must have an active BuildCookRun to use automated tests (@todo: eventually could add relevent data into each automated test, but maybe an overkill?)
		bool bHasActiveBuildCookRun = false;
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : BuildCookRuns)
		{
			bHasActiveBuildCookRun |= BuildCookRun->IsEnabled();
		}
		for (const ILauncherProfileAutomatedTestRef& AutomatedTest : GetAutomatedTests())
		{
			if (!bHasActiveBuildCookRun && AutomatedTest->IsEnabled())
			{
				ProfileValidation.Errors.Add(ELauncherProfileValidationErrors::AutomatedTestRequiredBuildCookRun);
				break;
			}
		}

		// validate build cook runs
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : BuildCookRuns)
		{
			Validate(BuildCookRun);
		}
	}

	void Validate(const ILauncherProfileBuildCookRunRef& BuildCookRun) 
	{
		if (!UATCommandValidation.Contains(BuildCookRun.ToSharedPtr()))
		{
			return;
		}

		FValidationData& ValidationData = UATCommandValidation.FindChecked(BuildCookRun.ToSharedPtr());
		ValidationData.Reset();

		// temp. fix : merge in the global validation errors so legacy PL can display them
		if (BuildCookRun == FirstBuildCookRun)
		{
			ValidationData.Errors = ProfileValidation.Errors;
		}

		// don't verify if this build command is disabled - it won't affect the build
		if (!BuildCookRun->IsEnabled())
		{
			return;
		}

		// Build: a build configuration must be selected
		if (BuildCookRun->GetBuildConfiguration() == EBuildConfiguration::Unknown)
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoBuildConfigurationSelected);
		}

		// Cook: at least one platform must be selected when cooking by the book
		if ((BuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook) && (BuildCookRun->GetCookedPlatforms().Num() == 0))
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoPlatformSelected);
		}

		// Cook: at least one culture must be selected when cooking by the book
		if ((BuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook) && (BuildCookRun->GetCookedCultures().Num() == 0))
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoCookedCulturesSelected);
		}

		// Deploy: a device group must be selected when deploying builds
		if ((BuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyToDevice) && !BuildCookRun->GetDeployedDeviceGroup(false).IsValid())
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::DeployedDeviceGroupRequired);
		}

		// Deploy: deployment by copying to devices requires cooking by the book unless the data is available from zen
		if ((BuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyToDevice) && ((BuildCookRun->GetCookMode() != ELauncherProfileCookModes::ByTheBook)&&(BuildCookRun->GetCookMode()!=ELauncherProfileCookModes::ByTheBookInEditor)) && !BuildCookRun->IsUsingZenPakStreaming() && !BuildCookRun->IsUsingZenStreaming() && !BuildCookRun->IsUsingPreStagedBuild())
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook);
		}

		// Deploy: deployment by copying a packaged build to devices requires a package dir
		if ((BuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyRepository) && (BuildCookRun->GetPackageDirectory() == TEXT("")))
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoPackageDirectorySpecified);
		}

		// Launch: custom launch roles are not supported yet
		if (BuildCookRun->GetLaunchMode() == ELauncherProfileLaunchModes::CustomRoles)
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::CustomRolesNotSupportedYet);
		}

		// Launch: when using custom launch roles, all roles must have a device assigned
		if (BuildCookRun->GetLaunchMode() == ELauncherProfileLaunchModes::CustomRoles)
		{
			for (int32 RoleIndex = 0; RoleIndex < BuildCookRun->GetLaunchRoles().Num(); ++RoleIndex)
			{
				if (BuildCookRun->GetLaunchRoles()[RoleIndex]->GetAssignedDevice().IsEmpty())
				{
					ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoLaunchRoleDeviceAssigned);
				
					break;
				}
			}
		}

		// Launch: Zen pak streaming requires deployment and launch
		if (BuildCookRun->IsUsingZenPakStreaming() && (BuildCookRun->GetLaunchMode() == ELauncherProfileLaunchModes::DoNotLaunch || BuildCookRun->GetDeploymentMode() != ELauncherProfileDeploymentModes::CopyToDevice))
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch);
		}

		bool bLegacyIterative = false;
		GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterative"), bLegacyIterative, GEditorIni);

		if (BuildCookRun->IsCookingUnversioned() && bLegacyIterative && BuildCookRun->IsCookingIncrementally() && ((BuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBook) || (BuildCookRun->GetCookMode() == ELauncherProfileCookModes::ByTheBookInEditor)))
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::UnversionedAndIncremental);
		}


		if ( (BuildCookRun->IsGeneratingPatch() || BuildCookRun->ShouldAddPatchLevel()) && (BuildCookRun->GetCookMode() != ELauncherProfileCookModes::ByTheBook) )
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::GeneratingPatchesCanOnlyRunFromByTheBookCookMode);
		}

		if (BuildCookRun->ShouldAddPatchLevel() && !BuildCookRun->IsGeneratingPatch() )
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::GeneratingMultiLevelPatchesRequiresGeneratePatch);
		}

		if (ShouldStageBaseReleasePaks() && BuildCookRun->GetBasedOnReleaseVersionName().IsEmpty()) // deprecated
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::StagingBaseReleasePaksWithoutABaseReleaseVersion);
		}

		if ( BuildCookRun->IsGeneratingChunks() && (BuildCookRun->GetCookMode() != ELauncherProfileCookModes::ByTheBook) )
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::GeneratingChunksRequiresCookByTheBook);
		}

		if (BuildCookRun->IsGeneratingChunks() && !BuildCookRun->IsPackingWithUnrealPak())
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::GeneratingChunksRequiresUnrealPak);
		}

		if (BuildCookRun->IsGenerateHttpChunkData() && !BuildCookRun->IsGeneratingChunks() && !BuildCookRun->IsCreatingDLC())
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::GeneratingHttpChunkDataRequiresGeneratingChunks);
		}

		if (BuildCookRun->IsGenerateHttpChunkData() && (BuildCookRun->GetHttpChunkDataReleaseName().IsEmpty() || !FPaths::DirectoryExists(*BuildCookRun->GetHttpChunkDataDirectory())))
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::GeneratingHttpChunkDataRequiresValidDirectoryAndName);
		}

		// Launch: when launching, all devices that the build is launched on must have content cooked for their platform
		if (BuildCookRun->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch && BuildCookRun->GetCookMode() != ELauncherProfileCookModes::OnTheFly && BuildCookRun->GetCookMode() != ELauncherProfileCookModes::OnTheFlyInEditor)
		{
			// @todo ensure that launched devices have cooked content
		}
		
		if ((BuildCookRun->GetCookMode() == ELauncherProfileCookModes::OnTheFly) || (BuildCookRun->GetCookMode() == ELauncherProfileCookModes::OnTheFlyInEditor))
		{
			if (BuildCookRun->GetBuildConfiguration() == EBuildConfiguration::Shipping)
			{
				// shipping doesn't support commandline options
				ValidationData.Errors.Add(ELauncherProfileValidationErrors::ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly);
			}

		}

		if (BuildCookRun->GetCookMode() == ELauncherProfileCookModes::OnTheFly)
		{

			for (auto const& CookedPlatform : BuildCookRun->GetCookedPlatforms())
			{
				if (CookedPlatform.Contains("Server"))
				{
					ValidationData.Errors.Add(ELauncherProfileValidationErrors::CookOnTheFlyDoesntSupportServer);
				}
			}
			

		}

		if (BuildCookRun->IsArchiving() && BuildCookRun->GetArchiveDirectory().IsEmpty())
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoArchiveDirectorySpecified);
		}

		if (BuildCookRun->IsUsingIoStore() && !BuildCookRun->IsPackingWithUnrealPak())
		{
			ValidationData.Errors.Add(ELauncherProfileValidationErrors::IoStoreRequiresPakFiles);
		}

		ValidateBuildTarget(BuildCookRun);
		ValidatePlatformSDKs(BuildCookRun);
		ValidateDeviceStatus(BuildCookRun);

		RefreshCustomWarningsAndErrors(BuildCookRun.ToSharedPtr());
	}

	
	void ValidatePlatformSDKs(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		if (!UATCommandValidation.Contains(BuildCookRun.ToSharedPtr()))
		{
			return;
		}


		FValidationData& ValidationData = UATCommandValidation.FindChecked(BuildCookRun.ToSharedPtr());

		ValidationData.Errors.Remove(ELauncherProfileValidationErrors::NoPlatformSDKInstalled);
		
		// Cook: ensure that all platform SDKs are installed
		if (BuildCookRun->GetCookedPlatforms().Num() > 0)
		{
			bool bProjectHasCode = false; // @todo: Does the project have any code?
			FString NotInstalledDocLink;
			for (auto PlatformName : BuildCookRun->GetCookedPlatforms())
			{
				const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);
				if(!Platform || !Platform->IsSdkInstalled(bProjectHasCode, NotInstalledDocLink))
				{
					ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoPlatformSDKInstalled);
					ILauncherServicesModule& LauncherServicesModule = FModuleManager::GetModuleChecked<ILauncherServicesModule>(TEXT("LauncherServices"));
					LauncherServicesModule.BroadcastLauncherServicesSDKNotInstalled(PlatformName, NotInstalledDocLink);
					if (!Platform)
					{
						BuildCookRun->RemoveCookedPlatform(PlatformName);
					}
					else
					{
						ValidationData.InvalidPlatform = PlatformName;
					}
					return;
				}
			}
		}
		
		// Deploy: ensure that all the target device SDKs are installed
		if ((BuildCookRun->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy) && BuildCookRun->GetDeployedDeviceGroup(false).IsValid())
		{
			const TArray<FString>& Devices = BuildCookRun->GetDeployedDeviceGroup(false)->GetDeviceIDs();
			for(auto DeviceId : Devices)
			{
				ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
				
				if (TargetDeviceServicesModule)
				{
					TSharedPtr<ITargetDeviceProxy> DeviceProxy = TargetDeviceServicesModule->GetDeviceProxyManager()->FindProxy(DeviceId);
					
					if(DeviceProxy.IsValid())
					{
						FString const& PlatformName = DeviceProxy->GetTargetPlatformName(DeviceProxy->GetTargetDeviceVariant(DeviceId));
						bool bProjectHasCode = false; // @todo: Does the project have any code?
						FString NotInstalledDocLink;
						const ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);
						if(!Platform || !Platform->IsSdkInstalled(bProjectHasCode, NotInstalledDocLink))
						{
							ValidationData.Errors.Add(ELauncherProfileValidationErrors::NoPlatformSDKInstalled);
							ILauncherServicesModule& LauncherServicesModule = FModuleManager::GetModuleChecked<ILauncherServicesModule>(TEXT("LauncherServices"));
							LauncherServicesModule.BroadcastLauncherServicesSDKNotInstalled(PlatformName, NotInstalledDocLink);
							BuildCookRun->GetDeployedDeviceGroup(false)->RemoveDevice(DeviceId);
							return;
						}
					}
				}
			}
		}
	}

	void ValidateDeviceStatus(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		FValidationData& ValidationData = UATCommandValidation.FindChecked(BuildCookRun.ToSharedPtr());

		ValidationData.Errors.Remove(ELauncherProfileValidationErrors::LaunchDeviceIsUnauthorized);

		if (BuildCookRun->GetDeployedDeviceGroup(false).IsValid())
		{
			const TArray<FString>& Devices = BuildCookRun->GetDeployedDeviceGroup(false)->GetDeviceIDs();
			for (auto DeviceId : Devices)
			{
				ITargetDeviceServicesModule* TargetDeviceServicesModule = static_cast<ITargetDeviceServicesModule*>(FModuleManager::Get().LoadModule(TEXT("TargetDeviceServices")));
				if (TargetDeviceServicesModule)
				{
					TSharedPtr<ITargetDeviceProxy> DeviceProxy = TargetDeviceServicesModule->GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(DeviceId);
					if (DeviceProxy.IsValid())
					{
						if (!DeviceProxy->IsAuthorized())
						{
							ValidationData.Errors.Add(ELauncherProfileValidationErrors::LaunchDeviceIsUnauthorized);
							return;
						}
					}
				}
			}
		}
	}

	void ValidateBuildTarget(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		// this is only necessary if the legacy project launcher was used to create complex build target situations
		// or if the legacy project launcher is being used	
		if (BuildCookRun->GetCookedPlatforms().Num() <= 1 && BuildCookRun->GetBuildTargets().Num() <= 1 && !BuildTargetOptionsChangedDelegate.IsBound() )
		{
			return;
		}
	
		FValidationData& ValidationData = UATCommandValidation.FindChecked(BuildCookRun.ToSharedPtr());

		bool bBuildTargetIsRequired = false;
		bool bBuildTargetIsSelected = false;

		TArray<FString> CurrentBuildTargets;
		if (BuildCookRun->HasBuildTargetSpecified())
		{
			CurrentBuildTargets = BuildCookRun->GetBuildTargets();
			if (CurrentBuildTargets.IsEmpty())
			{
				CurrentBuildTargets.Add(FString());
			}
		}
		else
		{
			CurrentBuildTargets.Add(LauncherProfileManager->GetBuildTarget());
		}


		TSet<EBuildTargetType> CookTargetTypes = GetCookTargetTypes(BuildCookRun);
		if (CookTargetTypes.Num() == 0)
		{
			CookTargetTypes.Add(EBuildTargetType::Game); // UAT defaults to 'Game' too
		}

		// check all build targets
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TArray<FString>& BuildTargetNames = HasProjectSpecified() ? ExplictBuildTargetNames : LauncherProfileManager->GetAllExplicitBuildTargetNames();	
		const TArray<FTargetInfo>& Targets = FDesktopPlatformModule::Get()->GetTargetsForProject(GetProjectPath());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		for (const FTargetInfo& Target : Targets)
		{
			if (CookTargetTypes.Contains(Target.Type))
			{
				for (const FString& BuildTarget : CurrentBuildTargets)
				{
					if (BuildTarget.IsEmpty())
					{
						// most target types can have a default build target specified in engine ini so no need to enforce this (UAT will give informative error if the ini isn't set up)
						bool bSupportsDefaultBuildTarget = (Target.Type != EBuildTargetType::Program); 

						if (!bSupportsDefaultBuildTarget && BuildTargetNames.Contains(Target.Name))
						{
							bBuildTargetIsRequired = true;
							break;
						}
					}
					else
					{
						if (Target.Name == BuildTarget)
						{
							bBuildTargetIsSelected = true;
							break;
						}
					}
				}
			}
		}

		for (const FString& BuildTarget : CurrentBuildTargets)
		{
			if (!BuildTarget.IsEmpty() && !bBuildTargetIsSelected)
			{
				ValidationData.Errors.Add(ELauncherProfileValidationErrors::BuildTargetCookVariantMismatch);
				break;
			}
		}

		if (bBuildTargetIsRequired)
		{
			if (BuildCookRun->HasBuildTargetSpecified())
			{
				ValidationData.Errors.Add(ELauncherProfileValidationErrors::BuildTargetIsRequired);
			}
			else
			{
				ValidationData.Errors.Add(ELauncherProfileValidationErrors::FallbackBuildTargetIsRequired);
			}
		}
	}


	void RefreshValidBuildTargets()
	{
		// this is only necessary if the legacy project launcher was used to create complex build target situations
		// or if the legacy project launcher is being used	
		bool bHasComplexBuildTargets = false;
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : GetBuildCookRunCommands())
		{
			if (BuildCookRun->GetCookedPlatforms().Num() > 1 || BuildCookRun->GetBuildTargets().Num() > 1)
			{
				bHasComplexBuildTargets = true;
				break;
			}
		}
		if (!bHasComplexBuildTargets && !BuildTargetOptionsChangedDelegate.IsBound())
		{
			return;
		}
		
		TArray<FString> LatestExplicitBuildTargetNames;

		// collect the build targets for the current project, filtered to what we are currently wanting to cook. Do not show fallback project's build targets
		if (HasProjectSpecified())
		{
			//TSet<EBuildTargetType> CookTargetTypes = GetCookTargetTypes(); // fixme: is it critical to have a 'get all cook target types' ?
			LatestExplicitBuildTargetNames = FGameProjectHelper::GetExplicitBuildTargetsForProject( GetProjectPath()/*, &CookTargetTypes*/ );
		}

		// notify listeners if the explicitly-required build targets have changed
		if (ExplictBuildTargetNames != LatestExplicitBuildTargetNames)
		{
			ExplictBuildTargetNames = LatestExplicitBuildTargetNames;
			BuildTargetOptionsChangedDelegate.Broadcast();
		}
	}


	TSet<EBuildTargetType> GetCookTargetTypes(const ILauncherProfileBuildCookRunRef& BuildCookRun) const
	{
		TSet<EBuildTargetType> CookTargetTypes;
		for ( const FString& Variant : BuildCookRun->GetCookedPlatforms() )
		{
			if (Variant.EndsWith(TEXT("Client")))
			{
				CookTargetTypes.Add(EBuildTargetType::Client);
			}
			else if (Variant.EndsWith(TEXT("Server")))
			{
				CookTargetTypes.Add(EBuildTargetType::Server);
			}
			else if (Variant.EndsWith(TEXT("Editor")))
			{
				CookTargetTypes.Add(EBuildTargetType::Editor);
			}
			else
			{
				CookTargetTypes.Add(EBuildTargetType::Game);
			}
		}

		return MoveTemp(CookTargetTypes);
	}

	void OnSelectedProjectChanged()
	{
		RefreshValidBuildTargets();
	}


private:

	FValidationData* GetValidationData(ILauncherProfileUATCommandPtr UATCommand)
	{
		// temp. fix : use the merged validation errors so legacy PL can display them
		if (!UATCommand.IsValid() && FirstBuildCookRun.IsValid())
		{
			UATCommand = FirstBuildCookRun;
		}

		if (UATCommand.IsValid())
		{
			return UATCommandValidation.Find(UATCommand);
		}
		else
		{
			return &ProfileValidation;
		}
	}

	const FValidationData* GetValidationData(ILauncherProfileUATCommandPtr UATCommand) const
	{
		// temp. fix : use the merged validation errors so legacy PL can display them
		if (!UATCommand.IsValid() && FirstBuildCookRun.IsValid())
		{
			UATCommand = FirstBuildCookRun;
		}

		if (UATCommand.IsValid())
		{
			return UATCommandValidation.Find(UATCommand);
		}
		else
		{
			return &ProfileValidation;
		}
	}


	FString GetUniqueUATCommandName( const TCHAR* Base ) const
	{
		int32 UniqueIndex = 2;
		FString UniqueName = Base;
		while (GetUATCommand(*UniqueName) != nullptr)
		{
			UniqueName = FString::Printf(TEXT("%s-%d"), Base, UniqueIndex++);
		}

		return UniqueName;
	}


	ILauncherProfileUATCommandRef CreateUATCommandInternal( const TCHAR* InternalName, const TCHAR* UserTypeName );
	ILauncherProfileAutomatedTestRef CreateAutomatedTestInternal( const TCHAR* InternalName, const TCHAR* UserTypeName );
	ILauncherProfileBuildCookRunRef CreateBuildCookRunInternal( const TCHAR* InternalName, const TCHAR* UserTypeName );
	ILauncherProfileUATCommandPtr CreateUATCommandByTypeNameInternal( const TCHAR* TypeName, const TCHAR* InternalName );
	ILauncherProfileBuildCookRunPtr LoadBuildCookRunInternal(const FJsonObject& Object, int32 Version);

	void AddUATCommandInternal( const ILauncherProfileUATCommandRef& UATCommand )
	{
		UATCommands.Add(UATCommand);
		UATCommandValidation.Add(UATCommand.ToSharedPtr());
	}


	void QueryZenSnapshotsForProject();
	TSharedPtr<UE::Zen::Build::FBuildListRetriever> GetBuildListRetriever();

	//  Holds a reference to the launcher profile manager.
	ILauncherProfileManagerRef LauncherProfileManager;

	
	// Holds a flag indicating whether UAT should be built
	bool BuildUAT;
	// This build stages pak files from the release version it is based on (deprecated)
	bool StageBaseReleasePaks;
	// Holds the profile's unique identifier.
	FGuid Id;
	// Holds the profile's name.
	FString Name;

	// Holds the profile's description.
	FString Description;
	// Holds a flag indicating whether the project is specified by this profile.
	bool ProjectSpecified;

	// Holds the full absolute path to the Unreal project used by this profile.
	FString FullProjectPath;

	// Holds the path that might be shareable between people.  Only works if the project is under the UE root.
	// otherwise this is an absolute path.
	FString ShareableProjectPath;

	// Profile validation
	FValidationData ProfileValidation;
	TMap<ILauncherProfileUATCommandPtr, FValidationData> UATCommandValidation;

	// Path to the editor executable to pass to UAT, for cooking, etc... May be empty.
	FString EditorExe;

	// Profile is for an internal project
	bool bNotForLicensees;

	// Update flash on device before running
	bool bShouldUpdateFlash;

	// Key-Value pairs for extsibility. Not user-facing.
	TMap<FString,FString> CustomStringProperties;
	TMap<FString, bool>   CustomBoolProperties;

	// Automated tests
	FString AutomatedTestBuildPath;
	bool bWantAutomatedTestBuild;

	// Extra UAT commands
	TArray<ILauncherProfileUATCommandRef> UATCommands;
	FOnLauncherProfileUATCommandChanged UATCommandAdded;
	FOnLauncherProfileUATCommandChanged UATCommandRemoved;

private:

	// Cache of the first BuildCookRun command in the list, for convenience & for backwards compatibilty with Legacy Project Launcher
	// NOTE: a profile is not guaranteed to have a BuildCookRun command - it can be deleted
	TSharedPtr<ILauncherProfileBuildCookRun> FirstBuildCookRun;

	// cook in the editor callbacks (not valid for any other cook mode)
	FIsCookFinishedDelegate IsCookFinishedDelegate;
	FCookCanceledDelegate CookCanceledDelegate;

	// Holds a delegate to be invoked when changing the device group to deploy to.
	FOnLauncherProfileDeployedDeviceGroupChanged DeployedDeviceGroupChangedDelegate;

	// Holds a delegate to be invoked when the project has changed
	FOnProfileProjectChanged ProjectChangedDelegate;

	// Holds a delegate to be invoked when the project build target options have changed
	FOnProfileBuildTargetOptionsChanged BuildTargetOptionsChangedDelegate;

	// Holds a delegate to be invoked when the profile is validated to gather custom warnings and errors
	FOnProfileCustomValidation CustomValidationDelegate;
	FOnProfileCustomUATCommandValidation CustomUATCommandValidationDelegate;

	// Cached build target options (not serialized)
	TArray<FString> ExplictBuildTargetNames;

	// Utility class instance used to query build list
	static TSharedPtr<UE::Zen::Build::FBuildListRetriever> BuildListRetriever;

	// Cache all builds for projects and platforms, shared between profiles
	typedef TMap<FString, int32> FPlatformToBuildsMap;
	static TMap<FString, FPlatformToBuildsMap> PerProjectBuilds;

	// friendship so UAT command specializations can access Validate()
	friend class FLauncherProfileBuildCookRun; // (also for legacy serialize)
	friend class FLauncherProfileAutomatedTest;
	friend class FLauncherProfileUATCommand;
};
