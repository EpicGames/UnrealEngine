// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfileAutomatedTest.h"
#include "ILauncherProfileBuildCookRun.h"
#include "Profiles/LauncherProfile.h"
#include "Dom/JsonObject.h"
#include "Internationalization/Internationalization.h"

class FLauncherProfileAutomatedTest final : public ILauncherProfileAutomatedTest, public TSharedFromThis<FLauncherProfileAutomatedTest>
{
public:
	static const TCHAR* TypeName;

	FLauncherProfileAutomatedTest( TSharedPtr<FLauncherProfile> InProfile, const TCHAR* InInternalName = TEXT(""), const TCHAR* InUserTypeName = TEXT("") ) 
		: Profile(InProfile)
		, InternalName(InInternalName)
		, UserTypeName(InUserTypeName)
	{
	}

	//UE_DEPRECATED(5.8, "please use the Json Save/Load API") - confuses linux build which thinks the entire class is deprecated
	FLauncherProfileAutomatedTest( FArchive& Archive )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Serialize(Archive);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	virtual void SetUATCommand( const TCHAR* InUATCommand ) override
	{
		UATCommand = InUATCommand;
		if (Profile)
		{
			Profile->Validate();
		}
	}

	virtual const FString& GetUATCommand() const override
	{
		return UATCommand;
	}

	virtual void SetTests( const TCHAR* InTests ) override
	{
		Tests = InTests;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual const FString& GetTests() const override
	{
		return Tests;
	}

	virtual void SetAdditionalUATCommandLine( const TCHAR* InAdditionalCommandLine ) override
	{
		UATCommandLine = InAdditionalCommandLine;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual const FString& GetAdditionalUATCommandLine() const override
	{
		return UATCommandLine;
	}

	virtual void SetEnabled( bool bInEnabled ) override
	{
		bEnabled = bInEnabled;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual bool IsEnabled() const override
	{
		return bEnabled;
	}

	virtual void SetOrder( int32 InOrder ) override
	{
		Order = InOrder;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual int32 GetOrder() const override
	{
		return Order;
	}

	virtual const TCHAR* GetInternalName() const override
	{
		return *InternalName;
	}

	virtual void SetDescription(const FString& NewDescription) override
	{
		Description = NewDescription;
		if (Profile)
		{
			Profile->Validate();
		}
	}

	virtual FString GetDescription() const
	{
		return Description;
	}

	virtual void SetUserTypeName(const FString& NewUserTypeName) override
	{
		UserTypeName = NewUserTypeName;
		if (Profile)
		{
			Profile->Validate();
		}
	}

	virtual FString GetUserTypeName() const override
	{
		return UserTypeName;
	}

	virtual ILauncherProfileAutomatedTestPtr AsAutomatedTest() override
	{
		return AsShared().ToSharedPtr();
	}

	virtual ILauncherProfileBuildCookRunPtr AsBuildCookRun() override
	{
		return nullptr;
	}

	enum class EVersion : uint32
	{
		Original = 0,
		FirstVersioned = 1,
		AddUserTypeName = 2,
		// add new items here

		Final,
		Latest = Final-1,
	};


	virtual bool Load(const FJsonObject& Object) override
	{
		EVersion Version = EVersion::Original;
		if (!Object.TryGetNumberField(TEXT("Version"), (uint32&)Version))
		{
			Version = EVersion::Original;
		}
		if (Version > EVersion::Latest)
		{
			return false;
		}

		InternalName          = Object.GetStringField(TEXT("InternalName"));
		UATCommand            = Object.GetStringField(TEXT("UATCommand"));
		Tests                 = Object.GetStringField(TEXT("Tests"));
		UATCommandLine        = Object.GetStringField(TEXT("AdditionalCommandLine"));
		bEnabled              = Object.GetBoolField(TEXT("Enabled"));

		if (Version >= EVersion::FirstVersioned)
		{
			Description = Object.GetStringField(TEXT("Description"));
			Order       = Object.GetIntegerField(TEXT("Order"));
		}
		else
		{
			Object.TryGetNumberField(TEXT("Order"), Order);
		}
		Order = FMath::Max(1, Order);

		UserTypeName = InternalName;
		if (Version >= EVersion::AddUserTypeName)
		{
			UserTypeName = Object.GetStringField(TEXT("UserTypeName"));
		}

		return true;
	}

	virtual void Save(TJsonWriter<>& Writer) override
	{
		Writer.WriteObjectStart();
		Writer.WriteValue(TEXT("Type"),                  TypeName);
		Writer.WriteValue(TEXT("Version"),               (uint32)EVersion::Latest);
		Writer.WriteValue(TEXT("InternalName"),          InternalName);
		Writer.WriteValue(TEXT("UATCommand"),            UATCommand);
		Writer.WriteValue(TEXT("Tests"),                 Tests);
		Writer.WriteValue(TEXT("AdditionalCommandLine"), UATCommandLine);
		Writer.WriteValue(TEXT("Enabled"),               bEnabled);
		Writer.WriteValue(TEXT("Order"),                 Order);
		Writer.WriteValue(TEXT("Description"),           Description);
		Writer.WriteValue(TEXT("UserTypeName"),          UserTypeName);
		Writer.WriteObjectEnd();
	}

	UE_DEPRECATED(5.8, "please use the Json Save/Load API")
	virtual void Serialize( FArchive& Archive ) override final
	{
		Archive << InternalName
				<< UATCommand
				<< Tests
				<< UATCommandLine
				<< bEnabled
				<< Order
		;
	}


private:
	TSharedPtr<FLauncherProfile> Profile;

	FString UATCommand = TEXT("RunUnreal");
	FString Tests; //-Test=xxx,yyy
	FString UATCommandLine;

	bool bEnabled = true;
	int32 Order = 1; // longer tests should generally have a higher order

	FString InternalName;
	FString UserTypeName;
	FString Description = NSLOCTEXT("FLauncherProfileAutomatedTest", "DefaultDescripton", "Automated Test").ToString();
};


