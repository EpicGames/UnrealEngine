// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "ILauncherProfile.h"
#include "ILauncherServicesModule.h"
#include "Misc/Paths.h"
#include "Launcher/LauncherProjectPath.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Culture.h"
#include "Misc/App.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceServicesModule.h"
#include "GameProjectHelper.h"
#include "Profiles/LauncherProfile.h"
#include "Profiles/LauncherProfileLaunchRole.h"
#include "Profiles/LauncherProfileVersion.h"
#include "Profiles/LauncherProfileBuildCookRun.h"
#include "PlatformInfo.h"
#include "TargetReceipt.h"
#include "DesktopPlatformModule.h"

class Error;

LAUNCHERSERVICES_API bool HasPromotedTarget(const TCHAR* BaseDir, const TCHAR* TargetName, const TCHAR* Platform, EBuildConfiguration Configuration, const TCHAR* Architecture);

inline bool TryGetDefaultTargetName(const FString& ProjectFile, EBuildTargetType TargetType, FString& OutTargetName)
{
	const TArray<FTargetInfo>& Targets = FDesktopPlatformModule::Get()->GetTargetsForProject(ProjectFile);
	for (const FTargetInfo& Target : Targets)
	{
		if (Target.Type == TargetType && Target.DefaultTarget.Get(true))
		{
			OutTargetName = Target.Name;
			return true;
		}
	}
	return false;
}


/**
 * Implements a BuildCookRun UAT command
 */
class FLauncherProfileBuildCookRun final
	: public ILauncherProfileBuildCookRun
	, public TSharedFromThis<FLauncherProfileBuildCookRun>
{
public:
	static const TCHAR* TypeName;

	/**
	* Gets the folder in which profile files are stored.
	*
	* @return The folder path.
	*/
	static FString GetProfileFolder(bool bNotForLicensees)
	{
		if (bNotForLicensees)
		{
			return FPaths::EngineDir() / TEXT("Restricted/NotForLicensees/Programs/UnrealFrontend/Profiles");
		}
		return FPaths::EngineDir() / TEXT("Programs/UnrealFrontend/Profiles");
	}

	/**
	 * Default constructor.
	 */
	FLauncherProfileBuildCookRun( ILauncherProfileManagerRef InProfileManager, TSharedPtr<FLauncherProfile> InProfile, const TCHAR* InInternalName = TEXT(""), const TCHAR* InUserTypeName = TEXT("") ) 
		: LauncherProfileManager(InProfileManager)		
		, DefaultLaunchRole(MakeShareable(new FLauncherProfileLaunchRole()))
		, Profile(InProfile)
		, InternalName(InInternalName)
		, UserTypeName(InUserTypeName)
	{ 
		SetDefaults();
		if (Profile)
		{
			Profile->ProjectChangedDelegate.AddRaw(this, &FLauncherProfileBuildCookRun::OnSelectedProjectChanged);
		}
	}



	/**
	 * Destructor.
	 */
	~FLauncherProfileBuildCookRun( ) 
	{
		if (DeployedDeviceGroup.IsValid())
		{
			DeployedDeviceGroup->OnDeviceAdded().Remove(OnLauncherDeviceGroupDeviceAddedDelegateHandle);
			DeployedDeviceGroup->OnDeviceRemoved().Remove(OnLauncherDeviceGroupDeviceRemoveDelegateHandle);
		}
		if (Profile)
		{
			Profile->ProjectChangedDelegate.RemoveAll(this);
		}

	}

	/**
	 * Gets the identifier of the device group to deploy to.
	 *
	 * This method is used internally by the profile manager to read the device group identifier after
	 * loading this profile from a file. The profile manager will use this identifier to locate the
	 * actual device group to deploy to.
	 *
	 * @return The device group identifier, or an invalid GUID if no group was set or deployment is disabled.
	 */
	virtual const FGuid& GetDeployedDeviceGroupId( ) const override
	{
		return DeployedDeviceGroupId;
	}

	//~ Begin ILauncherProfile Interface

	virtual void AddCookedCulture( const FString& CultureName ) override
	{
		CookedCultures.AddUnique(CultureName);

		Validate();
	}

	virtual void AddCookedMap( const FString& MapName ) override
	{
		CookedMaps.AddUnique(MapName);

		Validate();
	}

	virtual void AddCookedPlatform( const FString& PlatformName ) override
	{
		CookedPlatforms.AddUnique(PlatformName);

		Validate();
	}

	virtual void SetDefaultDeployPlatform(const FName PlatformName) override
	{
		
		DefaultDeployPlatform = PlatformName;	

		if (DeployedDeviceGroup.IsValid())
		{
			DeployedDeviceGroup->RemoveAllDevices();

			if (DefaultDeployPlatform != NAME_None)
			{
				TArray<TSharedPtr<ITargetDeviceProxy>> PlatformDeviceProxies;
				ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");
				const TSharedRef<ITargetDeviceProxyManager>& InDeviceProxyManager = TargetDeviceServicesModule.GetDeviceProxyManager();

				InDeviceProxyManager->GetProxies(NAME_None, true, PlatformDeviceProxies);

				TSharedPtr<ITargetDeviceProxy> DefaultPlatformDevice;
				for (int32 ProxyIndex = 0; ProxyIndex < PlatformDeviceProxies.Num(); ++ProxyIndex)
				{
					TSharedPtr<ITargetDeviceProxy> DeviceProxy = PlatformDeviceProxies[ProxyIndex];

					if (DeviceProxy->GetVanillaPlatformId(NAME_None) == DefaultDeployPlatform)
					{
						DefaultPlatformDevice = DeviceProxy;
						break;
					}
				}

				if (DefaultPlatformDevice.IsValid())
				{
					DeployedDeviceGroup->AddDevice(DefaultPlatformDevice->GetTargetDeviceId(NAME_None));
				}
			}
		}

		Validate();
	}

	virtual void ClearCookedCultures( ) override
	{
		if (CookedCultures.Num() > 0)
		{
			CookedCultures.Reset();

			Validate();
		}
	}

	virtual void ClearCookedMaps( ) override
	{
		if (CookedMaps.Num() > 0)
		{
			CookedMaps.Reset();

			Validate();
		}
	}

	virtual void ClearCookedPlatforms( ) override
	{
		if (CookedPlatforms.Num() > 0)
		{
			CookedPlatforms.Reset();

			Validate();
		}
	}

	virtual ILauncherProfileLaunchRolePtr CreateLaunchRole( ) override
	{
		ILauncherProfileLaunchRolePtr Role = MakeShareable(new FLauncherProfileLaunchRole());
			
		LaunchRoles.Add(Role);

		Validate();

		return Role;
	}

	virtual EBuildConfiguration GetBuildConfiguration( ) const override
	{
		return BuildConfiguration;
	}

	virtual bool HasBuildTargetSpecified() const override
	{
		return BuildTargetSpecified;
	}

	virtual EBuildConfiguration GetCookConfiguration( ) const override
	{
		return CookConfiguration;
	}

	virtual ELauncherProfileCookModes::Type GetCookMode( ) const override
	{
		return CookMode;
	}

	virtual const FString& GetCookOptions( ) const override
	{
		return CookOptions;
	}

	virtual const TArray<FString>& GetCookedCultures( ) const override
	{
		return CookedCultures;
	}

	virtual const bool GetSkipCookingEditorContent() const override
	{
		return bSkipCookingEditorContent;
	}

	virtual const TArray<FString>& GetCookedMaps( ) const override
	{
		return CookedMaps;
	}

	virtual const TArray<FString>& GetCookedPlatforms( ) const override
	{
		return CookedPlatforms;
	}

	virtual const ILauncherProfileLaunchRoleRef& GetDefaultLaunchRole( ) const override
	{
		return DefaultLaunchRole;
	}

	virtual ILauncherDeviceGroupPtr GetDeployedDeviceGroup( bool bRefreshDevices = true ) override
	{
		// setting the default platform will update the device group.  always do this when getting the group because
		// devices come in lazily through messages and can't be added properly at profile load.
		if (DefaultDeployPlatform != NAME_None && bRefreshDevices)
		{
			SetDefaultDeployPlatform(DefaultDeployPlatform);
		}
		return DeployedDeviceGroup;
	}

	virtual const FName GetDefaultDeployPlatform() const override
	{
		return DefaultDeployPlatform;
	}

	virtual bool IsGeneratingPatch() const override
	{
		return GeneratePatch;
	}

	virtual bool ShouldAddPatchLevel() const override
	{
		return AddPatchLevel;
	}

	virtual bool IsCreatingDLC() const override
	{
		return CreateDLC;
	}
	virtual void SetCreateDLC(bool InBuildDLC) override
	{
		CreateDLC = InBuildDLC;
	}

	virtual FString GetDLCName() const override
	{
		return DLCName;
	}
	virtual void SetDLCName(const FString& InDLCName) override
	{
		DLCName = InDLCName;
	}

	virtual bool IsDLCIncludingEngineContent() const
	{
		return DLCIncludeEngineContent;
	}
	virtual void SetDLCIncludeEngineContent(bool InDLCIncludeEngineContent)
	{
		DLCIncludeEngineContent = InDLCIncludeEngineContent;
	}



	virtual bool IsCreatingReleaseVersion() const override
	{
		return CreateReleaseVersion;
	}

	virtual void SetCreateReleaseVersion(bool InCreateReleaseVersion) override
	{
		CreateReleaseVersion = InCreateReleaseVersion;
	}

	virtual FString GetCreateReleaseVersionName() const override
	{
		return CreateReleaseVersionName;
	}

	virtual void SetCreateReleaseVersionName(const FString& InCreateReleaseVersionName) override
	{
		CreateReleaseVersionName = InCreateReleaseVersionName;
	}


	virtual FString GetBasedOnReleaseVersionName() const override
	{
		return BasedOnReleaseVersionName;
	}

	virtual void SetBasedOnReleaseVersionName(const FString& InBasedOnReleaseVersionName) override
	{
		BasedOnReleaseVersionName = InBasedOnReleaseVersionName;
	}

	virtual FString GetOriginalReleaseVersionName() const override
	{
		return OriginalReleaseVersionName; 
	}

	virtual void SetOriginalReleaseVersionName(const FString& InOriginalReleaseVersionName) override
	{
		OriginalReleaseVersionName = InOriginalReleaseVersionName;
	}

	virtual FString GetReferenceContainerGlobalFileName() const override
	{
		return ReferenceContainerGlobalFileName;
	}
	virtual void SetReferenceContainerGlobalFileName(const FString& InReferenceContainerGlobalFileName) override
	{
		ReferenceContainerGlobalFileName = InReferenceContainerGlobalFileName;
	}
	virtual FString GetReferenceContainerCryptoKeysFileName() const override
	{
		return ReferenceContainerCryptoKeysFileName;
	}
	virtual void SetReferenceContainerCryptoKeysFileName(const FString& InReferenceContainerCryptoKeysFileName) override
	{
		ReferenceContainerCryptoKeysFileName = InReferenceContainerCryptoKeysFileName;
	}

	virtual ELauncherProfileDeploymentModes::Type GetDeploymentMode( ) const override
	{
		return DeploymentMode;
	}

    virtual bool GetForceClose() const override
    {
        return ForceClose;
    }

	virtual ELauncherProfileLaunchModes::Type GetLaunchMode( ) const override
	{
		return LaunchMode;
	}

	virtual const TArray<ILauncherProfileLaunchRolePtr>& GetLaunchRoles( ) const override
	{
		return LaunchRoles;
	}

	virtual const int32 GetLaunchRolesFor( const FString& DeviceId, TArray<ILauncherProfileLaunchRolePtr>& OutRoles ) override
	{
		OutRoles.Empty();

		if (LaunchMode == ELauncherProfileLaunchModes::CustomRoles)
		{
			for (TArray<ILauncherProfileLaunchRolePtr>::TConstIterator It(LaunchRoles); It; ++It)
			{
				ILauncherProfileLaunchRolePtr Role = *It;

				if (Role->GetAssignedDevice() == DeviceId)
				{
					OutRoles.Add(Role);
				}
			}
		}
		else if (LaunchMode == ELauncherProfileLaunchModes::DefaultRole)
		{
			OutRoles.Add(DefaultLaunchRole);
		}

		return OutRoles.Num();
	}

	virtual FString GetDescription() const override
	{
		return Description;
	}

	virtual ELauncherProfilePackagingModes::Type GetPackagingMode( ) const override
	{
		return PackagingMode;
	}

	virtual FString GetPackageDirectory( ) const override
	{
		return PackageDir;
	}

	virtual bool IsArchiving( ) const override
	{
		return bArchive;
	}

	virtual FString GetArchiveDirectory( ) const override
	{
		return ArchiveDir;
	}

    virtual uint32 GetTimeout() const override
    {
        return Timeout;
    }

	virtual ELauncherProfileBuildModes::Type GetBuildMode() const override
	{
		return BuildMode;
	}

	virtual bool ShouldBuild() override
	{
		bool bBuild = true;
		if (BuildMode == ELauncherProfileBuildModes::DoNotBuild)
		{
			bBuild = false;
		}
		else if (BuildMode == ELauncherProfileBuildModes::Auto)
		{
			if (FApp::GetEngineIsPromotedBuild())
			{
				bBuild = false;

				TArray<FString> TargetPlatformNames = FindPlatforms();
				for (const FString& TargetPlatformName : TargetPlatformNames)
				{
					// Get the target we're building for
					const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(TargetPlatformName);
					const PlatformInfo::FTargetPlatformInfo& PlatformInfo = TargetPlatform->GetTargetPlatformInfo();

					// Figure out which target we're building
					FString ReceiptDir;
					FString TargetName;
					if (TryGetDefaultTargetName(FPaths::GetProjectFilePath(), PlatformInfo.PlatformType, TargetName))
					{
						ReceiptDir = FPaths::GetPath(FPaths::GetProjectFilePath());
					}
					else if (TryGetDefaultTargetName(FString(), PlatformInfo.PlatformType, TargetName))
					{
						FText Reason;
						if (TargetPlatform->RequiresTempTarget(false, BuildConfiguration, false, Reason))
						{
							UE_LOGF(LogLauncherProfile, Log, "Project requires temp target (%ls)", *Reason.ToString());
							ReceiptDir = FPaths::GetPath(FPaths::GetProjectFilePath());
						}
						else
						{
							UE_LOGF(LogLauncherProfile, Log, "Project does not require temp target");
							ReceiptDir = FPaths::EngineDir();
						}
					}
					else
					{
						UE_LOGF(LogLauncherProfile, Log, "Unable to find any targets for platform %ls - forcing build", *TargetPlatformName);
						bBuild = true;
						break;
					}

					// Check if the existing target is valid
					FString BuildPlatform = PlatformInfo.DataDrivenPlatformInfo->UBTPlatformString;
					if (!HasPromotedTarget(*ReceiptDir, *TargetName, *BuildPlatform, BuildConfiguration, nullptr))
					{
						bBuild = true;
						break;
					}
				}
			}
		}
		return bBuild;
	}

	virtual bool IsFastIterate() const override
	{
		return bFastIterate;
	}

	virtual FString GetAdditionalCommandLineParameters() const override
	{
		return AdditionalCommandLineParameters;
	}

	virtual FString GetAdditionalTargetCommandLineParameters( EBuildTargetType BuildTargetType ) const override
	{
		check( BuildTargetType != EBuildTargetType::Unknown);

		const FString* CommandLinePtr = AdditionalTargetCommandLineParameters.Find(BuildTargetType);
		return CommandLinePtr ? *CommandLinePtr : FString();
	}

	virtual bool IsCookingIncrementally( ) const override
	{
		if ( CookMode != ELauncherProfileCookModes::DoNotCook )
		{
			return CookIncremental;
		}
		return false;
	}

	virtual bool IsIterateSharedCookedBuild() const override
	{
		if ( CookMode != ELauncherProfileCookModes::DoNotCook )
		{
			return IterateSharedCookedBuild;
		}
		return false;
	}

	virtual bool IsCompressed() const override
	{
		return Compressed;
	}

	virtual bool IsEncryptingIniFiles() const override
	{
		return EncryptIniFiles;
	}

	virtual bool IsForDistribution() const override
	{
		return ForDistribution;
	}

	virtual bool IsCookingUnversioned( ) const override
	{
		return CookUnversioned;
	}

	virtual bool IsDeployablePlatform( const FString& PlatformName ) override
	{
		if (CookMode == ELauncherProfileCookModes::ByTheBook || CookMode == ELauncherProfileCookModes::ByTheBookInEditor)
		{
			return CookedPlatforms.Contains(PlatformName);
		}

		return true;
	}

	virtual bool IsDeployingIncrementally( ) const override
	{
		return DeployIncremental;
	}

	virtual bool IsFileServerHidden( ) const override
	{
		return HideFileServerWindow;
	}

	virtual bool IsFileServerStreaming( ) const override
	{
		return DeployStreamingServer;
	}

	virtual bool IsPackingWithUnrealPak( ) const  override
	{
		return DeployWithUnrealPak;
	}

	virtual bool IsIncludingPrerequisites() const override
	{
		return IncludePrerequisites;
	}

	virtual bool IsGeneratingChunks() const override
	{
		return bGenerateChunks;
	}

	virtual bool IsGenerateHttpChunkData() const override
	{
		return bGenerateHttpChunkData;
	}

	virtual FString GetHttpChunkDataDirectory() const override
	{
		return HttpChunkDataDirectory;
	}

	virtual FString GetHttpChunkDataReleaseName() const override
	{
		return HttpChunkDataReleaseName;
	}

	virtual void RemoveCookedCulture( const FString& CultureName ) override
	{
		CookedCultures.Remove(CultureName);

		Validate();
	}

	virtual void RemoveCookedMap( const FString& MapName ) override
	{
		CookedMaps.Remove(MapName);

		Validate();
	}

	virtual void RemoveCookedPlatform( const FString& PlatformName ) override
	{
		CookedPlatforms.Remove(PlatformName);

		Validate();
	}

	virtual void RemoveLaunchRole( const ILauncherProfileLaunchRoleRef& Role ) override
	{
		LaunchRoles.Remove(Role);

		Validate();
	}

	virtual void Save(TJsonWriter<>& Writer) override
	{
		if (DeployedDeviceGroup.IsValid())
		{
			DeployedDeviceGroupId = DeployedDeviceGroup->GetId();
		}
		else
		{
			DeployedDeviceGroupId = FGuid();
		}

		Writer.WriteObjectStart();

		// ILauncherProfileUATCommand items
		Writer.WriteValue("Type", TypeName);
		Writer.WriteValue("InternalName", InternalName);
		Writer.WriteValue("UserTypeName", UserTypeName);
		Writer.WriteValue("UATCommand", UATCommand);
		Writer.WriteValue("UATCommandLine", UATCommandLine);
		Writer.WriteValue("Enabled", bEnabled);
		Writer.WriteValue("Order", Order);
		Writer.WriteValue("Description", Description);

		Writer.WriteValue("Version", (uint32)EVersion::Latest);
		Writer.WriteValue("BuildConfiguration", (int32)BuildConfiguration);
		Writer.WriteValue("CookConfiguration", (int32)CookConfiguration);
		Writer.WriteValue("CookIncremental", CookIncremental);
		Writer.WriteValue("CookOptions", CookOptions);
		Writer.WriteValue("CookMode", CookMode);
		Writer.WriteValue("CookUnversioned", CookUnversioned);

		// write the cultures
		if (CookedCultures.Num() > 0)
		{
			Writer.WriteArrayStart("CookedCultures");
			for (auto Value : CookedCultures)
			{
				Writer.WriteValue(Value);
			}
			Writer.WriteArrayEnd();
		}

		// write the maps
		if (CookedMaps.Num() > 0)
		{
			Writer.WriteArrayStart("CookedMaps");
			for (auto Value : CookedMaps)
			{
				Writer.WriteValue(Value);
			}
			Writer.WriteArrayEnd();
		}

		// write the platforms
		if (CookedPlatforms.Num() > 0)
		{
			Writer.WriteArrayStart("CookedPlatforms");
			for (auto Value : CookedPlatforms)
			{
				Writer.WriteValue(Value);
			}
			Writer.WriteArrayEnd();
		}

		// write the build targets
		if (BuildTargets.Num() > 0)
		{
			Writer.WriteArrayStart("BuildTargets");
			for (auto Value : BuildTargets)
			{
				Writer.WriteValue(Value);
			}
			Writer.WriteArrayEnd();
		}

		Writer.WriteValue("DeployStreamingServer", DeployStreamingServer);
		Writer.WriteValue("DeployWithUnrealPak", DeployWithUnrealPak);
		Writer.WriteValue("DeployedDeviceGroupId", DeployedDeviceGroupId.ToString());
		Writer.WriteValue("DeploymentMode", DeploymentMode);
		Writer.WriteValue("HideFileServerWindow", HideFileServerWindow);
		Writer.WriteValue("LaunchMode", LaunchMode);
		Writer.WriteValue("PackagingMode", PackagingMode);
		Writer.WriteValue("PackageDir", PackageDir);
		Writer.WriteValue("BuildMode", BuildMode);
		Writer.WriteValue("ForceClose", ForceClose);
		Writer.WriteValue("Timeout", (int32)Timeout);
		Writer.WriteValue("Compressed", Compressed);
		Writer.WriteValue("EncryptIniFiles", EncryptIniFiles);
		Writer.WriteValue("ForDistribution", ForDistribution);
		Writer.WriteValue("DeployPlatform", DefaultDeployPlatform.ToString());
		Writer.WriteValue("SkipCookingEditorContent", bSkipCookingEditorContent);
		Writer.WriteValue("DeployIncremental", DeployIncremental);
		Writer.WriteValue("GeneratePatch", GeneratePatch);
		Writer.WriteValue("AddPatchLevel", AddPatchLevel);
		Writer.WriteValue("DLCIncludeEngineContent", DLCIncludeEngineContent);
		Writer.WriteValue("CreateReleaseVersion", CreateReleaseVersion);
		Writer.WriteValue("CreateReleaseVersionName", CreateReleaseVersionName);
		Writer.WriteValue("BasedOnReleaseVersionName", BasedOnReleaseVersionName);
		Writer.WriteValue("ReferenceContainerGlobalFileName", ReferenceContainerGlobalFileName);
		Writer.WriteValue("ReferenceContainerCryptoKeysFileName", ReferenceContainerCryptoKeysFileName);
		Writer.WriteValue("OriginalReleaseVersionName", OriginalReleaseVersionName);
		Writer.WriteValue("CreateDLC", CreateDLC);
		Writer.WriteValue("DLCName", DLCName);
		Writer.WriteValue("GenerateChunks", bGenerateChunks);
		Writer.WriteValue("GenerateHttpChunkData", bGenerateHttpChunkData);
		Writer.WriteValue("HttpChunkDataDirectory", HttpChunkDataDirectory);
		Writer.WriteValue("HttpChunkDataReleaseName", HttpChunkDataReleaseName);
		Writer.WriteValue("Archive", bArchive);
		Writer.WriteValue("ArchiveDirectory", ArchiveDir);
		Writer.WriteValue("AdditionalCommandLineParameters", AdditionalCommandLineParameters);
		Writer.WriteValue("IncludePrerequisites", IncludePrerequisites);
		Writer.WriteValue("UseIoStore", bUseIoStore);
		Writer.WriteValue("MakeBinaryConfig", bMakeBinaryConfig);
		Writer.WriteValue("BuildTargetSpecified", BuildTargetSpecified);
		Writer.WriteValue("UseZenStore", bUseZenStore);
		Writer.WriteValue("ImportingZenSnapshot", bImportingZenSnapshot);
		Writer.WriteValue("UseZenStreaming", bUseZenStreaming);
		Writer.WriteValue("UseZenPakStreaming", bUseZenPakStreaming);
		Writer.WriteValue("ZenPakStreamingPath", ZenPakStreamingPath);
		Writer.WriteValue("PreStaged", bUsePreStagedBuild);
		Writer.WriteValue("ClientArchitectures", ClientArchitectures);
		Writer.WriteValue("ServerArchitectures", ServerArchitectures);
		Writer.WriteValue("EditorArchitectures", EditorArchitectures);
		Writer.WriteValue("FastIterate", bFastIterate);

		if (AdditionalTargetCommandLineParameters.Num() > 0)
		{
			Writer.WriteObjectStart("AdditionalTargetCommandLineParameters");
			for (const TTuple<EBuildTargetType,FString>& Itr : AdditionalTargetCommandLineParameters)
			{
				Writer.WriteValue(LexToString(Itr.Key), Itr.Value);
			}
			Writer.WriteObjectEnd();
		}

		// serialize the default launch role
		DefaultLaunchRole->Save(Writer, TEXT("DefaultRole"));

		// serialize the launch roles
		if (LaunchRoles.Num())
		{
			Writer.WriteArrayStart("LaunchRoles");
			for (auto Value : LaunchRoles)
			{
				Value->Save(Writer);
			}
			Writer.WriteArrayEnd();
		}

		Writer.WriteObjectEnd();
	}


	TArray<FString> FindPlatforms()
	{
		TArray<FString> Platforms;
		if (GetCookMode() == ELauncherProfileCookModes::ByTheBook)
		{
			Platforms = GetCookedPlatforms();
		}

		// determine deployment platforms
		ILauncherDeviceGroupPtr DeviceGroup = GetDeployedDeviceGroup();
		FName Variant = NAME_None;

		// Loading the Device Proxy Manager to get the needed Device Manager.
		ITargetDeviceServicesModule& DeviceServiceModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>(TEXT("TargetDeviceServices"));
		TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager = DeviceServiceModule.GetDeviceProxyManager();

		if (DeviceGroup.IsValid() && Platforms.Num() < 1)
		{
			const TArray<FString>& Devices = DeviceGroup->GetDeviceIDs();
			// for each deployed device...
			for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
			{
				const FString& DeviceId = Devices[DeviceIndex];

				TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceId);

				if (DeviceProxy.IsValid())
				{
					// add the platform
					Variant = DeviceProxy->GetTargetDeviceVariant(DeviceId);
					Platforms.AddUnique(DeviceProxy->GetTargetPlatformName(Variant));
				}
			}
		}

		return Platforms;
	}

	enum class EVersion : uint32
	{
		PreInnerBuildCookRun=0,
		Original=1,
		AddUserTypeName=2,
		// add new items here

		Final,
		Latest = Final-1,
	};



	virtual bool Load(const FJsonObject& Object) override
	{
		return Load(Object, LAUNCHERSERVICES_FINAL);
	}

	bool Load(const FJsonObject& Object, int32 ProfileVersion)
	{
		// read the version
		// earlier versions of the launcher profile contained all the BuildCookRun parameters directly,
		// but since LAUNCHERSERVICES_INNERBUILDCOOKRUN, this is serialized in the launcher profile's UAT command list
		EVersion Version = EVersion::PreInnerBuildCookRun;
		if (ProfileVersion >= LAUNCHERSERVICES_INNERBUILDCOOKRUN)
		{
			Version = (EVersion)Object.GetNumberField(TEXT("Version"));
			if (Version > EVersion::Latest)
			{
				return false;
			}
		}

		if (Version >= EVersion::Original)
		{
			// ILauncherProfileUATCommand items
			InternalName          = Object.GetStringField(TEXT("InternalName"));
			UATCommand            = Object.GetStringField(TEXT("UATCommand"));
			UATCommandLine        = Object.GetStringField(TEXT("UATCommandLine"));
			bEnabled              = Object.GetBoolField(TEXT("Enabled"));
			Order                 = Object.GetIntegerField(TEXT("Order"));
			Description           = Object.GetStringField(TEXT("Description"));
		}
		UserTypeName = InternalName;
		if (Version >= EVersion::AddUserTypeName)
		{
			UserTypeName = Object.GetStringField(TEXT("UserTypeName"));
		}

		BuildConfiguration = (EBuildConfiguration)((int32)Object.GetNumberField(TEXT("BuildConfiguration")));
		CookConfiguration = (EBuildConfiguration)((int32)Object.GetNumberField(TEXT("CookConfiguration")));
		CookOptions = Object.GetStringField(TEXT("CookOptions"));
		CookMode = (TEnumAsByte<ELauncherProfileCookModes::Type>)((int32)Object.GetNumberField(TEXT("CookMode")));
		CookUnversioned = Object.GetBoolField(TEXT("CookUnversioned"));

		CookedCultures.Reset();
		const TArray<TSharedPtr<FJsonValue>>* Cultures = NULL;
		if (Object.TryGetArrayField(TEXT("CookedCultures"), Cultures))
		{
			for (auto Value : *Cultures)
			{
				CookedCultures.Add(Value->AsString());
			}
		}

		CookedMaps.Reset();
		const TArray<TSharedPtr<FJsonValue>>* Maps = NULL;
		if (Object.TryGetArrayField(TEXT("CookedMaps"), Maps))
		{
			for (auto Value : *Maps)
			{
				CookedMaps.Add(Value->AsString());
			}
		}

		CookedPlatforms.Reset();
		const TArray<TSharedPtr<FJsonValue>>* Platforms = NULL;
		if (Object.TryGetArrayField(TEXT("CookedPlatforms"), Platforms))
		{
			for (auto Value : *Platforms)
			{
				CookedPlatforms.Add(Value->AsString());
			}
		}

		DeployStreamingServer = Object.GetBoolField(TEXT("DeployStreamingServer"));
		DeployWithUnrealPak = Object.GetBoolField(TEXT("DeployWithUnrealPak"));
		FGuid::Parse(Object.GetStringField(TEXT("DeployedDeviceGroupId")), DeployedDeviceGroupId);
		DeploymentMode = (TEnumAsByte<ELauncherProfileDeploymentModes::Type>)((int32)Object.GetNumberField(TEXT("DeploymentMode")));
		HideFileServerWindow = Object.GetBoolField(TEXT("HideFileServerWindow"));
		LaunchMode = (TEnumAsByte<ELauncherProfileLaunchModes::Type>)((int32)Object.GetNumberField(TEXT("LaunchMode")));
		PackagingMode = (TEnumAsByte<ELauncherProfilePackagingModes::Type>)((int32)Object.GetNumberField(TEXT("PackagingMode")));
		PackageDir = Object.GetStringField(TEXT("PackageDir"));

		int64 BuildModeValue;
		if (Object.TryGetNumberField(TEXT("BuildMode"), BuildModeValue))
		{
			BuildMode = (TEnumAsByte<ELauncherProfileBuildModes::Type>)(int32)BuildModeValue;
		}
		else
		{
			BuildMode = Object.GetBoolField(TEXT("BuildGame")) ? ELauncherProfileBuildModes::Build : ELauncherProfileBuildModes::DoNotBuild;
		}

		ForceClose = Object.GetBoolField(TEXT("ForceClose"));
		Timeout = (uint32)Object.GetNumberField(TEXT("Timeout"));
		Compressed = Object.GetBoolField(TEXT("Compressed"));

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDENCRYPTINIFILES)
		{
			EncryptIniFiles = Object.GetBoolField(TEXT("EncryptIniFiles"));
			ForDistribution = Object.GetBoolField(TEXT("ForDistribution"));
		}
		else
		{
			EncryptIniFiles = false;
			ForDistribution = false;
		}

		DefaultDeployPlatform = *(Object.GetStringField(TEXT("DeployPlatform")));
		bSkipCookingEditorContent = Object.GetBoolField(TEXT("SkipCookingEditorContent"));
		DeployIncremental = Object.GetBoolField(TEXT("DeployIncremental"));
		GeneratePatch = Object.GetBoolField(TEXT("GeneratePatch"));

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDMULTILEVELPATCHING)
		{
			AddPatchLevel = Object.GetBoolField(TEXT("AddPatchLevel"));
		}
		else
		{
			AddPatchLevel = false;
		}

		DLCIncludeEngineContent = Object.GetBoolField(TEXT("DLCIncludeEngineContent"));
		CreateReleaseVersion = Object.GetBoolField(TEXT("CreateReleaseVersion"));
		CreateReleaseVersionName = Object.GetStringField(TEXT("CreateReleaseVersionName"));
		BasedOnReleaseVersionName = Object.GetStringField(TEXT("BasedOnReleaseVersionName"));

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDREFERENCECONTAINERS)
		{
			ReferenceContainerCryptoKeysFileName = Object.GetStringField(TEXT("ReferenceContainerCryptoKeysFileName"));
			ReferenceContainerGlobalFileName = Object.GetStringField(TEXT("ReferenceContainerGlobalFileName"));
		}
		else
		{
			ReferenceContainerCryptoKeysFileName.Empty();
			ReferenceContainerGlobalFileName.Empty();
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDORIGINALRELEASEVERSION)
		{
			OriginalReleaseVersionName = Object.GetStringField(TEXT("OriginalReleaseVersionName"));
		}
		else
		{
			OriginalReleaseVersionName.Empty();
		}

		CreateDLC = Object.GetBoolField(TEXT("CreateDLC"));
		DLCName = Object.GetStringField(TEXT("DLCName"));
		bGenerateChunks = Object.GetBoolField(TEXT("GenerateChunks"));
		bGenerateHttpChunkData = Object.GetBoolField(TEXT("GenerateHttpChunkData"));
		HttpChunkDataDirectory = Object.GetStringField(TEXT("HttpChunkDataDirectory"));
		HttpChunkDataReleaseName = Object.GetStringField(TEXT("HttpChunkDataReleaseName"));

		if (ProfileVersion >= LAUNCHERSERVICES_ADDARCHIVE)
		{
			bArchive = Object.GetBoolField(TEXT("Archive"));
			ArchiveDir = Object.GetStringField(TEXT("ArchiveDirectory"));
		}
		else
		{
			bArchive = false;
			ArchiveDir = TEXT("");
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDADDITIONALCOMMANDLINE)
		{
			AdditionalCommandLineParameters = Object.GetStringField(TEXT("AdditionalCommandLineParameters"));
		}
		else
		{
			AdditionalCommandLineParameters = TEXT("");
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDINCLUDEPREREQUISITES)
		{
			IncludePrerequisites = Object.GetBoolField(TEXT("IncludePrerequisites"));
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDUSEIOSTORE)
		{
			bUseIoStore = Object.GetBoolField(TEXT("UseIoStore"));
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDEDMAKEBINARYCONFIG)
		{
			bMakeBinaryConfig = Object.GetBoolField(TEXT("MakeBinaryConfig"));
		}

		FString BuildTargetName;
		if (ProfileVersion >= LAUNCHERSERVICES_ADDBUILDTARGETNAME)
		{
			BuildTargetSpecified = Object.GetBoolField(TEXT("BuildTargetSpecified"));
			if (ProfileVersion < LAUNCHERSERVICES_ADDMULTIPLEBUILDTARGETS)
			{
				BuildTargetName = Object.GetStringField(TEXT("BuildTargetName"));
				if (!BuildTargetName.IsEmpty())
				{
					BuildTargets.Add(BuildTargetName);
				}
			}
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDZENOPTIONS)
		{
			bUseZenStore = Object.GetBoolField(TEXT("UseZenStore"));
			bImportingZenSnapshot = Object.GetBoolField(TEXT("ImportingZenSnapshot"));
			bUseZenPakStreaming = Object.GetBoolField(TEXT("UseZenPakStreaming"));
			ZenPakStreamingPath = Object.GetStringField(TEXT("ZenPakStreamingPath"));
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDZENSTREAMINGOPTIONS)
		{
			bUseZenStreaming = Object.GetBoolField(TEXT("UseZenStreaming"));
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDBUILDARCHITECTURE)
		{
			ClientArchitectures.Reset();
			ServerArchitectures.Reset();
			EditorArchitectures.Reset();
			Object.TryGetStringArrayField(TEXT("ClientArchitectures"), ClientArchitectures);
			Object.TryGetStringArrayField(TEXT("ServerArchitectures"), ServerArchitectures);
			Object.TryGetStringArrayField(TEXT("EditorArchitectures"), EditorArchitectures);
		}

		if (ProfileVersion >= LAUNCHERSERVICES_INCREMENTALCOOKMODE && ProfileVersion < LAUNCHERSERVICES_REMOVEDINCREMENTALCOOKMODE)
		{
			ELauncherProfileIncrementalCookMode::Type IncrementalCookMode = (TEnumAsByte<ELauncherProfileIncrementalCookMode::Type>)((int32)Object.GetNumberField(TEXT("IncrementalCookMode")));
			CookIncremental = (IncrementalCookMode != ELauncherProfileIncrementalCookMode::None);
		}
		else if (ProfileVersion < LAUNCHERSERVICES_INCREMENTALCOOKMODE || ProfileVersion >= LAUNCHERSERVICES_REMOVEDINCREMENTALCOOKMODE)
		{
			CookIncremental = Object.GetBoolField(TEXT("CookIncremental"));
		}

		if (ProfileVersion >= LAUNCHERSERVICES_FASTITERATE)
		{
			bFastIterate = Object.GetBoolField(TEXT("FastIterate"));
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDMULTIPLEBUILDTARGETS)
		{
			BuildTargets.Reset();
			const TArray<TSharedPtr<FJsonValue>>* JsonValues = NULL;
			if (Object.TryGetArrayField(TEXT("BuildTargets"), JsonValues))
			{
				for (auto Value : *JsonValues)
				{
					BuildTargets.Add(Value->AsString());
				}
			}

			if (BuildTargets.Num() == 0 && !BuildTargetName.IsEmpty())
			{
				BuildTargets.Add(BuildTargetName);
			}


			AdditionalTargetCommandLineParameters.Reset();
			const TSharedPtr<FJsonObject>* AdditionalTargetCommandLineParametersObject;
			if (Object.TryGetObjectField(TEXT("AdditionalTargetCommandLineParameters"), AdditionalTargetCommandLineParametersObject))
			{
				for (const auto& KeyPairs : (*AdditionalTargetCommandLineParametersObject)->Values)
				{
					EBuildTargetType BuildTarget;
					if (LexTryParseString(BuildTarget, *KeyPairs.Key))
					{
						AdditionalTargetCommandLineParameters.Add(BuildTarget, KeyPairs.Value->AsString());
					}
				}
			}
		}

		if (ProfileVersion >= LAUNCHERSERVICES_ADDPRESTAGED)
		{
			bUsePreStagedBuild = Object.GetBoolField(TEXT("PreStaged"));
		}

		// load the default launch role
		TSharedPtr<FJsonObject> Role = Object.GetObjectField(TEXT("DefaultRole"));
		DefaultLaunchRole->Load(*(Role.Get()));

		// serialize the launch roles
		DeployedDeviceGroup.Reset();
		LaunchRoles.Reset();
		const TArray<TSharedPtr<FJsonValue>>* Roles = NULL;
		if (Object.TryGetArrayField(TEXT("LaunchRoles"), Roles))
		{
			for (auto Value : *Roles)
			{
				LaunchRoles.Add(MakeShareable(new FLauncherProfileLaunchRole(*(Value->AsObject().Get()))));
			}
		}

		if (DefaultDeployPlatform != NAME_None)
		{
			SetDefaultDeployPlatform(DefaultDeployPlatform);
		}

		ILauncherDeviceGroupPtr DeviceGroup = LauncherProfileManager->GetDeviceGroup(DeployedDeviceGroupId);
		if (!DeviceGroup.IsValid())
		{
			DeviceGroup = LauncherProfileManager->AddNewDeviceGroup();
		}
		SetDeployedDeviceGroup(DeviceGroup);

		Validate();

		return true;
	}

	virtual void SetDefaults( ) override
	{
		AdditionalCommandLineParameters = FString();

		// I don't use FApp::GetBuildConfiguration() because i don't want the act of running in debug the first time to cause 
		// profiles the user creates to be in debug. This will keep consistency.
		BuildConfiguration = EBuildConfiguration::Development;

		FInternationalization& I18N = FInternationalization::Get();

		// default build settings
		BuildMode = ELauncherProfileBuildModes::Auto;
		bFastIterate = false;
		BuildTargetSpecified = false;

		// default cook settings
		CookConfiguration = FApp::GetBuildConfiguration();
		CookMode = ELauncherProfileCookModes::OnTheFly;
		CookOptions = FString();
		CookIncremental = false;
		IterateSharedCookedBuild=false;
		CookUnversioned = true;
		Compressed = true;
		EncryptIniFiles = false;
		ForDistribution = false;
		CookedCultures.Reset();
		CookedCultures.Add(I18N.GetCurrentCulture()->GetName());
		CookedMaps.Reset();
		CookedPlatforms.Reset();
		BuildTargets.Reset();
		bSkipCookingEditorContent = false;
        ForceClose = true;
        Timeout = 60;

		bArchive = false;
		ArchiveDir = TEXT("");

		// default deploy settings
		DeployedDeviceGroup.Reset();
		DeploymentMode = ELauncherProfileDeploymentModes::CopyToDevice;
		DeployStreamingServer = false;
		IncludePrerequisites = false;
		DeployWithUnrealPak = false;
		DeployedDeviceGroupId = FGuid();
		HideFileServerWindow = false;
		DeployIncremental = false;

		CreateReleaseVersion = false;
		GeneratePatch = false;
		AddPatchLevel = false;
		CreateDLC = false;
		DLCIncludeEngineContent = false;

		bGenerateChunks = false;
		bGenerateHttpChunkData = false;
		HttpChunkDataDirectory = TEXT("");
		HttpChunkDataReleaseName = TEXT("");

		// default launch settings
		DefaultDeployPlatform = NAME_None;
		LaunchMode = ELauncherProfileLaunchModes::DefaultRole;
		DefaultLaunchRole->SetCommandLine(FString());
		DefaultLaunchRole->SetInitialCulture(I18N.GetCurrentCulture()->GetName());
		DefaultLaunchRole->SetInitialMap(FString());
		DefaultLaunchRole->SetName(TEXT("Default Role"));
		DefaultLaunchRole->SetInstanceType(ELauncherProfileRoleInstanceTypes::StandaloneClient);
		DefaultLaunchRole->SetVsyncEnabled(false);
		LaunchRoles.Reset();

		// default packaging settings
		PackagingMode = ELauncherProfilePackagingModes::DoNotPackage;

		bUseIoStore = false;
		bUseZenStore = false;
		bImportingZenSnapshot = false;
		bUseZenStreaming = false;
		bUseZenPakStreaming = false;
		bUsePreStagedBuild = false;
		bIsDeviceASimulator = false;
		bMakeBinaryConfig = false;

		Description = NSLOCTEXT("FLauncherProfileBuildCookRun", "DefaultDescripton", "Build, Cook & Run").ToString();
	}

	virtual void SetBuildMode(ELauncherProfileBuildModes::Type Mode) override
	{
		if (BuildMode != Mode)
		{
			BuildMode = Mode;

			Validate();
		}
	}

	virtual void SetFastIterate(bool Enable) override
	{
		bFastIterate = Enable;
	}
	
	virtual void SetAdditionalCommandLineParameters(const FString& Params) override
	{
		if (AdditionalCommandLineParameters != Params)
		{
			AdditionalCommandLineParameters = Params;

			Validate();
		}
	}

	virtual void SetAdditionalTargetCommandLineParameters(const FString& Params, EBuildTargetType BuildTargetType ) override
	{
		check( BuildTargetType != EBuildTargetType::Unknown);

		FString CurrentParams = GetAdditionalTargetCommandLineParameters(BuildTargetType);
		if (CurrentParams != Params)
		{
			if (Params.IsEmpty())
			{
				AdditionalTargetCommandLineParameters.Remove(BuildTargetType);
			}
			else
			{
				AdditionalTargetCommandLineParameters.Add(BuildTargetType, Params);
			}

			Validate();
		}
	}



	virtual void SetBuildConfiguration( EBuildConfiguration Configuration ) override
	{
		if (BuildConfiguration != Configuration)
		{
			BuildConfiguration = Configuration;

			Validate();
		}
	}

	virtual void SetBuildTargetSpecified(bool Specified) override
	{
		if (BuildTargetSpecified != Specified)
		{
			BuildTargetSpecified = Specified;
			Validate();
		}
	}

	virtual void SetCookConfiguration( EBuildConfiguration Configuration ) override
	{
		if (CookConfiguration != Configuration)
		{
			CookConfiguration = Configuration;

			Validate();
		}
	}

	virtual void SetCookMode( ELauncherProfileCookModes::Type Mode ) override
	{
		if (CookMode != Mode)
		{
			CookMode = Mode;

			Validate();
		}
	}

	virtual void SetCookOptions(const FString& Options) override
	{
		if (CookOptions != Options)
		{
			CookOptions = Options;

			Validate();
		}
	}

	virtual void SetSkipCookingEditorContent(const bool InSkipCookingEditorContent) override
	{
		if (bSkipCookingEditorContent != InSkipCookingEditorContent)
		{
			bSkipCookingEditorContent = InSkipCookingEditorContent;
			Validate();
		}
	}

	virtual void SetDeployWithUnrealPak( bool UseUnrealPak ) override
	{
		if (DeployWithUnrealPak != UseUnrealPak)
		{
			DeployWithUnrealPak = UseUnrealPak;

			Validate();
		}
	}

	virtual void SetGenerateChunks(bool bInGenerateChunks) override
	{
		if (bGenerateChunks != bInGenerateChunks)
		{
			bGenerateChunks = bInGenerateChunks;
			Validate();
		}
	}

	virtual void SetGenerateHttpChunkData(bool bInGenerateHttpChunkData) override
	{
		if (bGenerateHttpChunkData != bInGenerateHttpChunkData)
		{
			bGenerateHttpChunkData = bInGenerateHttpChunkData;
			Validate();
		}
	}

	virtual void SetHttpChunkDataDirectory(const FString& InHttpChunkDataDirectory) override
	{
		if (HttpChunkDataDirectory != InHttpChunkDataDirectory)
		{
			HttpChunkDataDirectory = InHttpChunkDataDirectory;
			Validate();
		}
	}

	virtual void SetHttpChunkDataReleaseName(const FString& InHttpChunkDataReleaseName) override
	{
		if (HttpChunkDataReleaseName != InHttpChunkDataReleaseName)
		{
			HttpChunkDataReleaseName = InHttpChunkDataReleaseName;
			Validate();
		}
	}

	virtual void SetDeployedDeviceGroup( const ILauncherDeviceGroupPtr& DeviceGroup ) override
	{
		if(DeployedDeviceGroup.IsValid())
		{
			DeployedDeviceGroup->OnDeviceAdded().Remove(OnLauncherDeviceGroupDeviceAddedDelegateHandle);
			DeployedDeviceGroup->OnDeviceRemoved().Remove(OnLauncherDeviceGroupDeviceRemoveDelegateHandle);
		}
		DeployedDeviceGroup = DeviceGroup;
		if (DeployedDeviceGroup.IsValid())
		{
			OnLauncherDeviceGroupDeviceAddedDelegateHandle   = DeployedDeviceGroup->OnDeviceAdded().AddRaw(this, &FLauncherProfileBuildCookRun::OnLauncherDeviceGroupDeviceAdded);
			OnLauncherDeviceGroupDeviceRemoveDelegateHandle  = DeployedDeviceGroup->OnDeviceRemoved().AddRaw(this, &FLauncherProfileBuildCookRun::OnLauncherDeviceGroupDeviceRemove);
			DeployedDeviceGroupId = DeployedDeviceGroup->GetId();
		}
		else
		{
			DeployedDeviceGroupId = FGuid();
		}

		if (DefaultDeployPlatform != NAME_None)
		{
			SetDefaultDeployPlatform(DefaultDeployPlatform);
		}

		Validate();
	}

	virtual void SetDeploymentMode( ELauncherProfileDeploymentModes::Type Mode ) override
	{
		if (DeploymentMode != Mode)
		{
			DeploymentMode = Mode;

			Validate();
		}
	}

    virtual void SetForceClose( bool Close ) override
    {
        if (ForceClose != Close)
        {
            ForceClose = Close;
            Validate();
        }
    }
    
	virtual void SetHideFileServerWindow( bool Hide ) override
	{
		HideFileServerWindow = Hide;
	}

	virtual void SetIncrementalCooking( bool Incremental ) override
	{
		if (Incremental != CookIncremental)
		{
			CookIncremental = Incremental;
			Validate();
		}
	}

	virtual void SetIterateSharedCookedBuild( bool SharedCookedBuild ) override
	{
		if (IterateSharedCookedBuild != SharedCookedBuild)
		{
			IterateSharedCookedBuild = SharedCookedBuild;

			Validate();
		}
	}

	virtual void SetCompressed( bool Enabled ) override
	{
		if (Compressed != Enabled)
		{
			Compressed = Enabled;

			Validate();
		}
	}

	virtual void SetForDistribution(bool Enabled)override
	{
		if (ForDistribution != Enabled)
		{
			ForDistribution = Enabled;

			Validate();
		}
	}

	virtual void SetEncryptingIniFiles(bool Enabled) override
	{
		if (EncryptIniFiles != Enabled)
		{
			EncryptIniFiles = Enabled;

			Validate();
		}
	}

	virtual void SetIncrementalDeploying( bool Incremental ) override
	{
		if (DeployIncremental != Incremental)
		{
			DeployIncremental = Incremental;

			Validate();
		}
	}

	virtual void SetLaunchMode( ELauncherProfileLaunchModes::Type Mode ) override
	{
		if (LaunchMode != Mode)
		{
			LaunchMode = Mode;

			Validate();
		}
	}

	virtual void SetDescription(const FString& NewDescription) override
	{
		Description = NewDescription;
	}

	virtual void SetPackagingMode( ELauncherProfilePackagingModes::Type Mode ) override
	{
		if (PackagingMode != Mode)
		{
			PackagingMode = Mode;

			Validate();
		}
	}

	virtual void SetPackageDirectory( const FString& Dir ) override
	{
		if (PackageDir != Dir)
		{
			PackageDir = Dir;

			Validate();
		}
	}

	virtual void SetArchive( bool bInArchive ) override
	{
		if (bInArchive != bArchive)
		{
			bArchive = bInArchive;

			Validate();
		}
	}

	virtual void SetArchiveDirectory( const FString& Dir ) override
	{
		if (ArchiveDir != Dir)
		{
			ArchiveDir = Dir;

			Validate();
		}
	}

	virtual void SetStreamingFileServer( bool Streaming ) override
	{
		if (DeployStreamingServer != Streaming)
		{
			DeployStreamingServer = Streaming;

			Validate();
		}
	}

	virtual void SetIncludePrerequisites(bool InValue) override
	{
		if (IncludePrerequisites != InValue)
		{
			IncludePrerequisites = InValue;

			Validate();
		}
	}

    virtual void SetTimeout( uint32 InTime ) override
    {
        if (Timeout != InTime)
        {
            Timeout = InTime;
            
            Validate();
        }
    }
    
	virtual void SetUnversionedCooking( bool Unversioned ) override
	{
		if (CookUnversioned != Unversioned)
		{
			CookUnversioned = Unversioned;

			Validate();
		}
	}

	virtual void SetGeneratePatch( bool InGeneratePatch ) override
	{
		GeneratePatch = InGeneratePatch;
	}

	virtual void SetAddPatchLevel( bool InAddPatchLevel) override
	{
		AddPatchLevel = InAddPatchLevel;
	}

	virtual void SetUseIoStore(bool bInUseIoStore) override
	{
		bUseIoStore = bInUseIoStore;

		if (bUseIoStore)
		{
			SetDeployWithUnrealPak(true);
		}

		Validate();
	}

	virtual bool IsUsingIoStore() const override
	{
		return bUseIoStore;
	}

	virtual void SetUseZenStore(bool bInUseZenStore) override
	{
		bUseZenStore = bInUseZenStore;

		Validate();
	}

	virtual bool IsUsingZenStore() const override
	{
		return bUseZenStore;
	}

	virtual void SetImportingZenSnapshot( bool Import ) override
	{
		if (bImportingZenSnapshot != Import)
		{
			bImportingZenSnapshot = Import;

			Validate();
		}
	}

	virtual bool IsImportingZenSnapshot() const override
	{
		return bImportingZenSnapshot;
	}

	virtual bool IsUsingZenStreaming() const override
	{
		return bUseZenStreaming;
	}

	virtual void SetUseZenStreaming( bool UseZenStreaming) override
	{
		if (bUseZenStreaming != UseZenStreaming)
		{
			bUseZenStreaming = UseZenStreaming;

			Validate();
		}
	}

	virtual void SetUseZenPakStreaming( bool UseZenPakStreaming ) override
	{
		if (bUseZenPakStreaming != UseZenPakStreaming)
		{
			bUseZenPakStreaming = UseZenPakStreaming;

			Validate();
		}
	}

	virtual bool IsUsingZenPakStreaming() const override
	{
		if (CookMode == ELauncherProfileCookModes::DoNotCook)
		{
			return bUseZenPakStreaming;
		}
		return false;
	}

	virtual void SetZenPakStreamingPath( const FString& Path ) override
	{
		if (ZenPakStreamingPath != Path)
		{
			ZenPakStreamingPath = Path;

			Validate();
		}
	}

	virtual FString GetZenPakStreamingPath() const override
	{
		return ZenPakStreamingPath;
	}

	virtual void SetUsePreStagedBuild( bool UsePreStagedBuild ) override
	{
		if (bUsePreStagedBuild != UsePreStagedBuild)
		{
			bUsePreStagedBuild = UsePreStagedBuild;
			Validate();
		}
	}

	virtual bool IsUsingPreStagedBuild() const override
	{
		return bUsePreStagedBuild;
	}


	virtual void SetDeviceIsASimulator(bool bInIsDeviceASimualtor) override
	{
		bIsDeviceASimulator = bInIsDeviceASimualtor;
	}

	/**
	 * Is the Launch device actually a simulator?
	 */
	virtual bool IsDeviceASimulator() const override
	{
		return bIsDeviceASimulator;
	}

	virtual void SetMakeBinaryConfig(bool bInMakeBinaryConfig) override
	{
		bMakeBinaryConfig = bInMakeBinaryConfig;
	}

	virtual bool MakeBinaryConfig() const override
	{
		return bMakeBinaryConfig;
	}

	virtual void SetClientArchitectures( const TArray<FString>& InArchitectures )
	{
		ClientArchitectures = InArchitectures;
	}

	virtual const TArray<FString>& GetClientArchitectures() const override
	{
		return ClientArchitectures;
	}

	virtual void SetServerArchitectures( const TArray<FString>& InArchitectures )
	{
		ServerArchitectures = InArchitectures;
	}

	virtual const TArray<FString>& GetServerArchitectures() const override
	{
		return ServerArchitectures;
	}

	virtual void SetEditorArchitectures( const TArray<FString>& InArchitectures )
	{
		EditorArchitectures = InArchitectures;
	}

	virtual const TArray<FString>& GetEditorArchitectures() const override
	{
		return EditorArchitectures;
	}







	virtual TArray<FString> GetBuildTargets() const
	{
		if (BuildTargetSpecified)
		{
			return BuildTargets;
		}
		else
		{
			return TArray<FString>{LauncherProfileManager->GetBuildTarget()};
		}
	}

	virtual void AddBuildTarget( const FString& InBuildTarget )
	{
		BuildTargets.AddUnique(InBuildTarget);
		Validate();
	}

	virtual void RemoveBuildTarget( const FString& InBuildTarget )
	{
		BuildTargets.Remove(InBuildTarget);
		Validate();
	}

	virtual void ClearBuildTargets()
	{
		BuildTargets.Reset();
		Validate();
	}


	//~ End ILauncherProfileBuildCookRun Interface


	// ILauncherProfileUATCommand interface
	virtual void SetUATCommand( const TCHAR* InUATCommand ) override
	{
		UATCommand = InUATCommand;
		if (Profile)
		{
			Profile->Validate();
		}
	}

	virtual const FString& GetUATCommand() const override
	{
		return UATCommand;
	}

	virtual void SetAdditionalUATCommandLine( const TCHAR* InAdditionalCommandLine ) override
	{
		UATCommandLine = InAdditionalCommandLine;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual const FString& GetAdditionalUATCommandLine() const override
	{
		return UATCommandLine;
	}

	virtual void SetEnabled( bool bInEnabled ) override
	{
		bEnabled = bInEnabled;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual bool IsEnabled() const override
	{
		return bEnabled;
	}

	virtual void SetOrder( int32 InOrder ) override
	{
		Order = InOrder;
		if (Profile)
		{
			Profile->Validate();
		}
	}
	virtual int32 GetOrder() const override
	{
		return Order;
	}

	virtual const TCHAR* GetInternalName() const override
	{
		return *InternalName;
	}

	virtual void SetUserTypeName(const FString& NewUserTypeName) override
	{
		UserTypeName = NewUserTypeName;
		if (Profile)
		{
			Profile->Validate();
		}
	}

	virtual FString GetUserTypeName() const override
	{
		return UserTypeName;
	}

	virtual ILauncherProfileAutomatedTestPtr AsAutomatedTest() override
	{
		return nullptr;
	}

	virtual ILauncherProfileBuildCookRunPtr AsBuildCookRun() override
	{
		return AsShared().ToSharedPtr();
	}

protected:

	void Validate()
	{
		if (Profile)
		{
			Profile->Validate(AsShared());
		}
	}

	void OnLauncherDeviceGroupDeviceAdded(const ILauncherDeviceGroupRef& DeviceGroup, const FString& DeviceId)
	{
		if( Profile && DeviceGroup == DeployedDeviceGroup )
		{
			Profile->ValidatePlatformSDKs(AsShared());
		}
	}
	
	void OnLauncherDeviceGroupDeviceRemove(const ILauncherDeviceGroupRef& DeviceGroup, const FString& DeviceId)
	{
		if( Profile && DeviceGroup == DeployedDeviceGroup )
		{
			Profile->ValidatePlatformSDKs(AsShared());
		}
	}

	void OnSelectedProjectChanged()
	{
		check(Profile);
		BuildTargets.Reset();
		BuildTargetSpecified = Profile->HasProjectSpecified(); // if this is for 'Any Project' then it should use the fallback build target
	}

private:
	//  Holds a reference to the launcher profile manager.
	ILauncherProfileManagerRef LauncherProfileManager;

	// Holds the desired build configuration (only used if creating new builds).
	EBuildConfiguration BuildConfiguration;

	// Holds a flag indicating whether the build target is specified by this profile.
	bool BuildTargetSpecified;

	// Holds the names of the build targets (matching a .cs file) to build. Needed when multiple targets of a type exist
	TArray<FString> BuildTargets;

	// Holds the build mode.
	// Holds the build configuration name of the cooker.
	EBuildConfiguration CookConfiguration;

	// Holds additional cooker command line options.
	FString CookOptions;

	// Holds the cooking mode.
	TEnumAsByte<ELauncherProfileCookModes::Type> CookMode;

	// Holds the build mode
	TEnumAsByte<ELauncherProfileBuildModes::Type> BuildMode;

	// Holds a flag indicating whether fast iteration is enabled 
	bool bFastIterate;
	
	// Generate compressed content
	bool Compressed;

	// encrypt ini files in the pak file
	bool EncryptIniFiles;
	// is this build for distribution
	bool ForDistribution;
	
	// Holds a flag indicating whether only modified content should be cooked.
	bool CookIncremental;

	// hold a flag indicating if we want to iterate on the shared cooked build
	bool IterateSharedCookedBuild;

	// Holds a flag indicating whether packages should be saved without a version.
	bool CookUnversioned;

	bool bSkipCookingEditorContent;

	// This setting is used only if cooking by the book (only used if cooking by the book).
	TArray<FString> CookedCultures;

	// Holds the collection of maps to cook (only used if cooking by the book).
	TArray<FString> CookedMaps;

	// Holds the platforms to include in the build (only used if creating new builds).
	TArray<FString> CookedPlatforms;

	// Holds the platforms to deploy to if no specific devices were chosen for deploy.
	FName DefaultDeployPlatform;

	// Holds the default role (only used if launch mode is DefaultRole).
	ILauncherProfileLaunchRoleRef DefaultLaunchRole;

	// Holds a flag indicating whether a streaming file server should be used.
	bool DeployStreamingServer;

	// Holds a flag indicating whether content should be packaged with UnrealPak.
	bool DeployWithUnrealPak;

	// Flag to indicate if game prerequisites should be included
	bool IncludePrerequisites;

	// Flag indicating if content should be split into chunks
	bool bGenerateChunks;
	
	// Flag indicating if chunked content should be used to generate HTTPChunkInstall data
	bool bGenerateHttpChunkData;
	
	// Where to store HTTPChunkInstall data
	FString HttpChunkDataDirectory;
	
	// Version name of the HTTPChunkInstall data
	FString HttpChunkDataReleaseName;

	// if present, iostore container creation will try to use existing compressed blocks
	// instead of compressing new ones, to avoid patches from compressor version changes,
	// and to speed up iostore container creation time. See IoStoreUtilities.cpp ReferenceContainerGlobalFileName.
	FString ReferenceContainerGlobalFileName;

	// If ReferenceContainerGlobalFileName refers to encrypted containers, this is the filename of
	// the json file containing the keys.
	FString ReferenceContainerCryptoKeysFileName;

	// create a release version of the content (this can be used to base dlc / patches from)
	bool CreateReleaseVersion;

	// name of the release version
	FString CreateReleaseVersionName;

	// name of the release version to base this dlc / patch on
	FString BasedOnReleaseVersionName;

	// name of the original release version
	FString OriginalReleaseVersionName;

	// This build generate a patch based on some source content seealso PatchSourceContentPath
	bool GeneratePatch;

	// This build generates a new tier patch file for modified content
	bool AddPatchLevel;

	// This build will cook content for dlc See also DLCName
	bool CreateDLC;

	// name of the dlc we are going to build (the name of the dlc plugin)
	FString DLCName;

	// should the dlc include engine content in the current dlc 
	//  engine content which was not referenced by original release
	//  otherwise error on any access of engine content during dlc cook
	bool DLCIncludeEngineContent;

	// Holds a flag indicating whether to use incremental deployment
	bool DeployIncremental;

	// Holds the device group to deploy to.
	ILauncherDeviceGroupPtr DeployedDeviceGroup;

	// Delegate handles for registered DeployedDeviceGroup event handlers.
	FDelegateHandle OnLauncherDeviceGroupDeviceAddedDelegateHandle;
	FDelegateHandle OnLauncherDeviceGroupDeviceRemoveDelegateHandle;

	// Holds the identifier of the deployed device group.
	FGuid DeployedDeviceGroupId;

	// Holds the deployment mode.
	TEnumAsByte<ELauncherProfileDeploymentModes::Type> DeploymentMode;

	// Holds a flag indicating whether the file server's console window should be hidden.
	bool HideFileServerWindow;

	// Holds the launch mode.
	TEnumAsByte<ELauncherProfileLaunchModes::Type> LaunchMode;

	// Holds the launch roles (only used if launch mode is UsingRoles).
	TArray<ILauncherProfileLaunchRolePtr> LaunchRoles;

	// Holds the profile's description.
	FString Description;

	// Holds the packaging mode.
	TEnumAsByte<ELauncherProfilePackagingModes::Type> PackagingMode;

	// Holds the package directory
	FString PackageDir;

	// Whether to run archive step	
	bool bArchive;

	// Holds the archive directory
	FString ArchiveDir;

	// Holds the full path to a folder to use for Zen pak streaming
	FString ZenPakStreamingPath;

	// Whether to use a pre-staged build
	bool bUsePreStagedBuild;

    // Holds the time out time for the cook on the fly server
    uint32 Timeout;

    // Holds the close value for the cook on the fly server
    bool ForceClose;

	// Additional command line parameters to set for the application when it launches
	FString AdditionalCommandLineParameters;

	// Additional command line parameters to set for a given build target type when it launches
	TMap<EBuildTargetType,FString> AdditionalTargetCommandLineParameters;

	// Use I/O store.
	bool bUseIoStore;
	
	// Use Zen storage server
	bool bUseZenStore;

	// Import a Zen snapshot before cooking
	bool bImportingZenSnapshot;

	bool bUseZenStreaming;

	// Use Zen pak streaming
	bool bUseZenPakStreaming;

	// Make binary config.
	bool bMakeBinaryConfig;

	// Is the launch device actually a Simulator
	bool bIsDeviceASimulator;

	// Architectures to build (x64, arm64 etc)
	TArray<FString> ClientArchitectures;
	TArray<FString> ServerArchitectures;
	TArray<FString> EditorArchitectures;

private:
	TSharedPtr<FLauncherProfile> Profile;

	FString UATCommand = TEXT("BuildCookRun");
	FString UATCommandLine;

	bool bEnabled = true;
	int32 Order = 0;

	FString InternalName;
	FString UserTypeName;

	// only for backwards compatibility in FLauncherProfile::Serialize
	friend class FLauncherProfile;
};
