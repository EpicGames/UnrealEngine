// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfileLaunchRole.h"
#include "Dom/JsonObject.h"

class FLauncherProfileLaunchRole final
	: public ILauncherProfileLaunchRole
{
public:

	/** Default constructor. */
	FLauncherProfileLaunchRole(  )
		: InstanceType(ELauncherProfileRoleInstanceTypes::StandaloneClient)
		, Name(TEXT("Unnamed Role"))
	{ }

	/**
	 * Creates and initializes a new instance from the given archive.
	 *
	 * @param Archive - The archive to serialize from.
	 */
	UE_DEPRECATED(5.8, "please use the Json Save/Load API")
	FLauncherProfileLaunchRole( FArchive& Archive )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Serialize(Archive);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FLauncherProfileLaunchRole(const FJsonObject& Object)
	{
		Load(Object);
	}

public:

	//~ Begin ILauncherProfileLaunchRole Interface

	virtual const FString& GetAssignedDevice( ) const override
	{
		return AssignedDevice;
	}

	virtual const FString& GetUATCommandLine( ) const override
	{
		return CommandLine;
	}

	virtual const FString& GetInitialCulture( ) const override
	{
		return InitialCulture;
	}

	virtual const FString& GetInitialMap( ) const override
	{
		return InitialMapName;
	}

	virtual ELauncherProfileRoleInstanceTypes::Type GetInstanceType( ) const override
	{
		return InstanceType;
	}

	virtual const FString& GetName( ) const override
	{
		return Name;
	}

	virtual bool IsVsyncEnabled( ) const override
	{
		return VsyncEnabled;
	}

	virtual void Load(const FJsonObject& Object)
	{
		AssignedDevice = Object.GetStringField(TEXT("AssignedDevice"));
		CommandLine = Object.GetStringField(TEXT("CommandLine"));
		DeviceId = Object.GetStringField(TEXT("DeviceId"));
		InitialCulture = Object.GetStringField(TEXT("InitialCulture"));
		InitialMapName = Object.GetStringField(TEXT("InitialMapName"));
		Name = Object.GetStringField(TEXT("Name"));
		InstanceType = (TEnumAsByte<ELauncherProfileRoleInstanceTypes::Type>)((int32)Object.GetNumberField(TEXT("InstanceType")));
		VsyncEnabled = Object.GetBoolField(TEXT("VsyncEnabled"));
	}

	virtual void Save(TJsonWriter<>& Writer, const TCHAR* InName = TEXT(""))
	{
		Writer.WriteObjectStart(InName);
		Writer.WriteValue(TEXT("AssignedDevice"), AssignedDevice);
		Writer.WriteValue(TEXT("CommandLine"), CommandLine);
		Writer.WriteValue(TEXT("DeviceId"), DeviceId);
		Writer.WriteValue(TEXT("InitialCulture"), InitialCulture);
		Writer.WriteValue(TEXT("InitialMapName"), InitialMapName);
		Writer.WriteValue(TEXT("Name"), InName);
		Writer.WriteValue(TEXT("InstanceType"), InstanceType);
		Writer.WriteValue(TEXT("VsyncEnabled"), VsyncEnabled);
		Writer.WriteObjectEnd();
	}

	UE_DEPRECATED(5.8, "please use the Json Save/Load API")
	virtual void Serialize( FArchive& Archive ) override final
	{
		Archive << AssignedDevice
				<< CommandLine
				<< DeviceId
				<< InitialCulture
				<< InitialMapName
				<< Name
				<< InstanceType
				<< VsyncEnabled;
	}

	virtual void SetCommandLine( const FString& NewCommandLine ) override
	{
		CommandLine = NewCommandLine;
	}

	virtual void SetInitialCulture( const FString& CultureName ) override
	{
		InitialCulture = CultureName;
	}

	virtual void SetInitialMap( const FString& MapName ) override
	{
		InitialMapName = MapName;
	}

	virtual void SetInstanceType( ELauncherProfileRoleInstanceTypes::Type InInstanceType ) override
	{
		InstanceType = InInstanceType;
	}

	virtual void SetName( const FString& NewName ) override
	{
		Name = NewName;
	}

	virtual void SetVsyncEnabled( bool Enabled ) override
	{
		VsyncEnabled = Enabled;
	}

	//~ End ILauncherProfileLaunchRole Interface

private:

	// Holds the identifier of the device that is assigned to this role.
	FString AssignedDevice;

	// Holds optional command line parameters.
	FString CommandLine;

	// Holds the identifier of the device that is assigned to this role.
	FString DeviceId;

	// Holds the initial localization culture to launch with.
	FString InitialCulture;

	// Holds the name of the map to launch.
	FString InitialMapName;

	// Holds the role instance type.
	TEnumAsByte<ELauncherProfileRoleInstanceTypes::Type> InstanceType;

	// Holds the role's name.
	FString Name;

	// Holds a flag indicating whether VSync should be enabled.
	bool VsyncEnabled;
};
