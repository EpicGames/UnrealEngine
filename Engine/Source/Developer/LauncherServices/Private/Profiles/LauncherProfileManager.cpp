// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profiles/LauncherProfileManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Launcher/LauncherProjectPath.h"
#include "Launcher/LauncherWorker.h" // just for MakeBuildCookRunParamsForProjectCustomBuild
#include "Profiles/LauncherDeviceGroup.h"
#include "Profiles/LauncherProfile.h"
#include "GameProjectHelper.h"

LAUNCHERSERVICES_API bool HasPromotedTarget(const TCHAR* BaseDir, const TCHAR* TargetName, const TCHAR* Platform, EBuildConfiguration Configuration, const TCHAR* Architecture)
{
	// Get the path to the receipt, and check it exists
	const FString ReceiptPath = FTargetReceipt::GetDefaultPath(BaseDir, TargetName, Platform, Configuration, Architecture);
	if (!FPaths::FileExists(ReceiptPath))
	{
		UE_LOGF(LogLauncherProfile, Log, "Unable to use promoted target - %ls does not exist.", *ReceiptPath);
		return false;
	}

	// Read the receipt for this target
	FTargetReceipt Receipt;
	if (!Receipt.Read(ReceiptPath))
	{
		UE_LOGF(LogLauncherProfile, Log, "Unable to use promoted target - cannot read %ls", *ReceiptPath);
		return false;
	}

	// Check the receipt is for a promoted build
	if (!Receipt.Version.IsPromotedBuild)
	{
		UE_LOGF(LogLauncherProfile, Log, "Unable to use promoted target - receipt %ls is not for a promoted target", *ReceiptPath);
		return false;
	}

	// Make sure it matches the current build info
	FEngineVersion ReceiptVersion = Receipt.Version.GetEngineVersion();
	FEngineVersion CurrentVersion = FEngineVersion::Current();
	if (!ReceiptVersion.ExactMatch(CurrentVersion))
	{
		UE_LOGF(LogLauncherProfile, Log, "Unable to use promoted target - receipt version (%ls) is not exact match with current engine version (%ls)", *ReceiptVersion.ToString(), *CurrentVersion.ToString());
		return false;
	}

	// Print the matching target info
	UE_LOGF(LogLauncherProfile, Log, "Found promoted target with matching version at %ls", *ReceiptPath);
	return true;
}

/* ILauncherProfileManager structors
 *****************************************************************************/

FLauncherProfileManager::FLauncherProfileManager()
{
	ConfigFileName = FPaths::EngineSavedDir() / TEXT("Config") / TEXT("ProjectLauncher") / TEXT("UserSettings.ini");
	FConfigContext::ReadSingleIntoGConfig().Load(*ConfigFileName);
}

/* ILauncherProfileManager interface
 *****************************************************************************/

void FLauncherProfileManager::Load()
{
	LoadDeviceGroups();
	LoadProfiles();
}

void FLauncherProfileManager::AddDeviceGroup( const ILauncherDeviceGroupRef& DeviceGroup )
{
	if (!DeviceGroups.Contains(DeviceGroup))
	{
		// replace the existing device group
		ILauncherDeviceGroupPtr ExistingGroup = GetDeviceGroup(DeviceGroup->GetId());

		if (ExistingGroup.IsValid())
		{
			RemoveDeviceGroup(ExistingGroup.ToSharedRef());
		}

		// add the new device group
		DeviceGroups.Add(DeviceGroup);
		SaveDeviceGroups();

		DeviceGroupAddedDelegate.Broadcast(DeviceGroup);
	}
}


ILauncherDeviceGroupRef FLauncherProfileManager::AddNewDeviceGroup( )
{
	ILauncherDeviceGroupRef  NewGroup = MakeShareable(new FLauncherDeviceGroup(FGuid::NewGuid(), FString::Printf(TEXT("New Group %d"), DeviceGroups.Num())));

	AddDeviceGroup(NewGroup);

	return NewGroup;
}


ILauncherDeviceGroupRef FLauncherProfileManager::CreateUnmanagedDeviceGroup()
{
	ILauncherDeviceGroupRef  NewGroup = MakeShareable(new FLauncherDeviceGroup(FGuid::NewGuid(), TEXT("Simple Group")));
	return NewGroup;
}


ILauncherSimpleProfilePtr FLauncherProfileManager::FindOrAddSimpleProfile(const FString& DeviceName)
{
	// replace the existing profile
	ILauncherSimpleProfilePtr SimpleProfile = FindSimpleProfile(DeviceName);
	if (!SimpleProfile.IsValid())
	{
		SimpleProfile = MakeShareable(new FLauncherSimpleProfile(DeviceName));
		SimpleProfiles.Add(SimpleProfile);
	}
	
	return SimpleProfile;
}


ILauncherSimpleProfilePtr FLauncherProfileManager::FindSimpleProfile(const FString& DeviceName)
{
	for (int32 ProfileIndex = 0; ProfileIndex < SimpleProfiles.Num(); ++ProfileIndex)
	{
		ILauncherSimpleProfilePtr SimpleProfile = SimpleProfiles[ProfileIndex];

		if (SimpleProfile->GetDeviceName() == DeviceName)
		{
			return SimpleProfile;
		}
	}

	return nullptr;
}

ILauncherProfileRef FLauncherProfileManager::AddNewProfile()
{
	// find a unique name for the profile.
	int32 ProfileIndex = SavedProfiles.Num();
	FString ProfileName = FString::Printf(TEXT("New Profile %d"), ProfileIndex);

	for (int32 Index = 0; Index < SavedProfiles.Num(); ++Index)
	{
		if (SavedProfiles[Index]->GetName() == ProfileName)
		{
			ProfileName = FString::Printf(TEXT("New Profile %d"), ++ProfileIndex);
			Index = -1;

			continue;
		}
	}

	// create and add the profile
	TSharedRef<FLauncherProfile> NewProfile = MakeShared<FLauncherProfile>(AsShared(), FGuid::NewGuid(), ProfileName);
	NewProfile->SetDefaults();

	AddProfile(NewProfile);

	SaveJSONProfile(NewProfile);

	return NewProfile;
}

ILauncherProfileRef FLauncherProfileManager::CreateUnsavedProfile(FString ProfileName)
{
	// create and return the profile
	TSharedRef<FLauncherProfile> NewProfile = MakeShared<FLauncherProfile>(AsShared(), FGuid(), ProfileName);
	NewProfile->SetDefaults();
	
	AllProfiles.Add(NewProfile);
	
	return NewProfile;
}


void FLauncherProfileManager::AddProfile( const ILauncherProfileRef& Profile )
{
	if (!SavedProfiles.Contains(Profile))
	{
		// replace the existing profile
		ILauncherProfilePtr ExistingProfile = GetProfile(Profile->GetId());

		if (ExistingProfile.IsValid())
		{
			RemoveProfile(ExistingProfile.ToSharedRef());
		}

		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			if (!BuildCookRun->GetDeployedDeviceGroup().IsValid())
			{
				BuildCookRun->SetDeployedDeviceGroup(AddNewDeviceGroup());
			}
		}

		// add the new profile
		SavedProfiles.Add(Profile);
		AllProfiles.Add(Profile);

		ProfileAddedDelegate.Broadcast(Profile);
	}
}


ILauncherProfilePtr FLauncherProfileManager::FindProfile( const FString& ProfileName )
{
	for (int32 ProfileIndex = 0; ProfileIndex < SavedProfiles.Num(); ++ProfileIndex)
	{
		ILauncherProfilePtr Profile = SavedProfiles[ProfileIndex];

		if (Profile->GetName() == ProfileName)
		{
			return Profile;
		}
	}

	return nullptr;
}


const TArray<ILauncherDeviceGroupPtr>& FLauncherProfileManager::GetAllDeviceGroups( ) const
{
	return DeviceGroups;
}


const TArray<ILauncherProfilePtr>& FLauncherProfileManager::GetAllProfiles( ) const
{
	return SavedProfiles;
}


ILauncherDeviceGroupPtr FLauncherProfileManager::GetDeviceGroup( const FGuid& GroupId ) const
{
	for (int32 GroupIndex = 0; GroupIndex < DeviceGroups.Num(); ++GroupIndex)
	{
		const ILauncherDeviceGroupPtr& Group = DeviceGroups[GroupIndex];

		if (Group->GetId() == GroupId)
		{
			return Group;
		}
	}

	return nullptr;
}


ILauncherProfilePtr FLauncherProfileManager::GetProfile( const FGuid& ProfileId ) const
{
	for (int32 ProfileIndex = 0; ProfileIndex < SavedProfiles.Num(); ++ProfileIndex)
	{
		ILauncherProfilePtr Profile = SavedProfiles[ProfileIndex];

		if (Profile->GetId() == ProfileId)
		{
			return Profile;
		}
	}

	return nullptr;
}


ILauncherProfilePtr FLauncherProfileManager::LoadProfile( FArchive& Archive )
{
	TSharedRef<FLauncherProfile> Profile = MakeShared<FLauncherProfile>(AsShared());
	Profile->SetDefaults();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Profile->Serialize(Archive))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			ILauncherDeviceGroupPtr DeviceGroup = BuildCookRun->GetDeployedDeviceGroup();
			if (!DeviceGroup.IsValid())
			{
				DeviceGroup = AddNewDeviceGroup();	
			}
			BuildCookRun->SetDeployedDeviceGroup(DeviceGroup);
		}

		return Profile;
	}

	return nullptr;
}

ILauncherProfilePtr FLauncherProfileManager::LoadJSONProfile(FString ProfileFile)
{
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *ProfileFile))
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject> Object;
	TSharedRef<TJsonReader<> > Reader = TJsonReaderFactory<>::Create(FileContents);
	if (!FJsonSerializer::Deserialize(Reader, Object) || !Object.IsValid())
	{
		return nullptr;
	}

	TSharedRef<FLauncherProfile> Profile = MakeShared<FLauncherProfile>(AsShared());
	Profile->SetDefaults();

	if (Profile->Load(*(Object.Get())))
	{
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			ILauncherDeviceGroupPtr DeviceGroup = GetDeviceGroup(BuildCookRun->GetDeployedDeviceGroupId());
			if (!DeviceGroup.IsValid())
			{
				DeviceGroup = AddNewDeviceGroup();
			}
			BuildCookRun->SetDeployedDeviceGroup(DeviceGroup);
		}

		return Profile;
	}

	return nullptr;
}



void FLauncherProfileManager::LoadSettings( )
{
	LoadDeviceGroups();
	LoadProfiles();
}


void FLauncherProfileManager::RemoveDeviceGroup( const ILauncherDeviceGroupRef& DeviceGroup )
{
	if (DeviceGroups.Remove(DeviceGroup) > 0)
	{
		SaveDeviceGroups();

		DeviceGroupRemovedDelegate.Broadcast(DeviceGroup);
	}
}


void FLauncherProfileManager::RemoveSimpleProfile(const ILauncherSimpleProfileRef& SimpleProfile)
{
	if (SimpleProfiles.Remove(SimpleProfile) > 0)
	{
		// delete the persisted simple profile on disk
		FString SimpleProfileFileName = FLauncherProfile::GetProfileFolder(false) / SimpleProfile->GetDeviceName() + TEXT(".uslp");
		IFileManager::Get().Delete(*SimpleProfileFileName);
	}
}


void FLauncherProfileManager::RemoveProfile( const ILauncherProfileRef& Profile )
{
	AllProfiles.Remove(Profile);
	if (SavedProfiles.Remove(Profile) > 0)
	{
		if (Profile->GetId().IsValid())
		{
			// delete the persisted profile on disk
			FString ProfileFileName = Profile->GetFilePath();

			// delete the profile
			IFileManager::Get().Delete(*ProfileFileName);

			ProfileRemovedDelegate.Broadcast(Profile);
		}
	}
}


bool FLauncherProfileManager::SaveProfile(const ILauncherProfileRef& Profile)
{
	if (Profile->GetId().IsValid())
	{
		FString ProfileFileName = Profile->GetFilePath();
		FArchive* ProfileFileWriter = IFileManager::Get().CreateFileWriter(*ProfileFileName);

		if (ProfileFileWriter != nullptr)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Profile->Serialize(*ProfileFileWriter);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			delete ProfileFileWriter;

			return true;
		}
	}
	return false;
}


bool FLauncherProfileManager::SaveJSONProfile(const ILauncherProfileRef& Profile)
{
	if (Profile->GetId().IsValid())
	{
		FString Text;
		TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&Text);
		Profile->Save(Writer.Get());
		Writer->Close();
		return FFileHelper::SaveStringToFile(Text, *Profile->GetFilePath());
	}
	return false;
}

void FLauncherProfileManager::ChangeProfileName(const ILauncherProfileRef& Profile, FString Name)
{
	FString OldName = Profile->GetName();
	FString OldProfileFileName = Profile->GetFilePath();

	//change name and save to new location
	Profile->SetName(Name);
	if (SaveJSONProfile(Profile))
	{
		//delete the old profile if the location moved.  File names should be uppercase so this compare works on case sensitive and insensitive platforms
		if (OldProfileFileName.Compare(Profile->GetFilePath()) != 0)
		{
			
			IFileManager::Get().Delete(*OldProfileFileName);
		}
	}
	else
	{
		//if we couldn't save successfully, change the name back to keep files/profiles matching.
		Profile->SetName(OldName);
	}	
}

void FLauncherProfileManager::RegisterProfileWizard(const ILauncherProfileWizardPtr& InProfileWizard)
{
	ProfileWizards.Add(InProfileWizard);
}

void FLauncherProfileManager::UnregisterProfileWizard(const ILauncherProfileWizardPtr& InProfileWizard)
{
	ProfileWizards.Remove(InProfileWizard);
}

const TArray<ILauncherProfileWizardPtr>& FLauncherProfileManager::GetProfileWizards() const
{
	return ProfileWizards;
}

void FLauncherProfileManager::SaveSettings( )
{
	SaveDeviceGroups(true);
	SaveSimpleProfiles();
	SaveProfiles();
}

FString FLauncherProfileManager::GetProjectName() const
{
	return FLauncherProjectPath::GetProjectName(ProjectPath);
}

FString FLauncherProfileManager::GetProjectBasePath() const
{
	return FLauncherProjectPath::GetProjectBasePath(ProjectPath);
}

FString FLauncherProfileManager::GetProjectPath() const
{
	return ProjectPath;
}

void FLauncherProfileManager::SetProjectPath(const FString& InProjectPath)
{
	if (ProjectPath != InProjectPath)
	{
		ProjectPath = InProjectPath;

		SetBuildTarget(FString());

		for (ILauncherProfilePtr Profile : AllProfiles)
		{
			if (Profile.IsValid())
			{
				Profile->FallbackProjectUpdated();
			}
		}
		
		ProjectChangedDelegate.Broadcast();
	}
}

FString FLauncherProfileManager::GetBuildTarget() const
{
	return BuildTarget;
}

void FLauncherProfileManager::SetBuildTarget( const FString& InBuildTarget )
{
	if (BuildTarget != InBuildTarget)
	{
		BuildTarget = InBuildTarget;
		for (ILauncherProfilePtr Profile : AllProfiles)
		{
			if (Profile.IsValid())
			{
				Profile->FallbackBuildTargetUpdated();
			}
		}
	}
}

const TArray<FString> FLauncherProfileManager::GetAllExplicitBuildTargetNames() const
{
	return FGameProjectHelper::GetExplicitBuildTargetsForProject(ProjectPath);
}


void FLauncherProfileManager::LoadDeviceGroups( )
{
	if (GConfig != nullptr)
	{
		for (int Pass = 0; Pass < 2; Pass++)
		{
			const FConfigSection* LoadedDeviceGroups = (Pass == 0)
				? GConfig->GetSection(TEXT("Launcher.DeviceGroups"), false, GEngineIni)     // legacy location (differs per-project & per-program)
				: GConfig->GetSection(TEXT("Launcher.DeviceGroups"), false, ConfigFileName) // new location (common)
			;

			if (LoadedDeviceGroups != nullptr)
			{
				// parse the configuration file entries into device groups
				for (FConfigSection::TConstIterator It(*LoadedDeviceGroups); It; ++It)
				{
					if (It.Key() == TEXT("DeviceGroup"))
					{
						ILauncherDeviceGroupPtr DeviceGroup = ParseDeviceGroup(It.Value().GetValue());
						if (DeviceGroup.IsValid())
						{
							DeviceGroups.Add(DeviceGroup);
						}
					}
				}
			}
		}
	}
}


void FLauncherProfileManager::LoadProfiles( )
{
	TArray<FString> ProfileFileNames;

	//load and move legacy profiles
	{
		IFileManager::Get().FindFilesRecursive(ProfileFileNames, *GetLegacyProfileFolder(), TEXT("*.ulp"), true, false);
		for (TArray<FString>::TConstIterator It(ProfileFileNames); It; ++It)
		{
			FString ProfileFilePath = *It;
			FArchive* ProfileFileReader = IFileManager::Get().CreateFileReader(*ProfileFilePath);

			if (ProfileFileReader != nullptr)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ILauncherProfilePtr LoadedProfile = LoadProfile(*ProfileFileReader);
				delete ProfileFileReader;

				//re-save profile to new location
				if (LoadedProfile.IsValid())
				{
					SaveProfile(LoadedProfile.ToSharedRef());
				}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				//delete legacy profile.
				IFileManager::Get().Delete(*ProfileFilePath);				
			}
		}
	}

	// 0 = normal, 1 = NotForLicensees
	for (int32 Pass = 0; Pass < 2; Pass++)
	{
		//load and re-save legacy profiles
		IFileManager::Get().FindFilesRecursive(ProfileFileNames, *FLauncherProfile::GetProfileFolder(Pass == 1), TEXT("*.ulp"), true, false);
		for (TArray<FString>::TConstIterator It(ProfileFileNames); It; ++It)
		{
			FString ProfileFilePath = *It;
			FArchive* ProfileFileReader = IFileManager::Get().CreateFileReader(*ProfileFilePath);

			if (ProfileFileReader != nullptr)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ILauncherProfilePtr LoadedProfile = LoadProfile(*ProfileFileReader);
				delete ProfileFileReader;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				//re-save profile to the new format
				if (LoadedProfile.IsValid())
				{
					if (Pass == 1)
					{
						LoadedProfile->SetNotForLicensees();
					}
					SaveJSONProfile(LoadedProfile.ToSharedRef());
				}

				//delete legacy profile.
				IFileManager::Get().Delete(*ProfileFilePath);
			}
		}
	}

	// 0 = normal, 1 = NotForLicensees
	for (int32 Pass = 0; Pass < 2; Pass++)
	{
		ProfileFileNames.Reset();
		IFileManager::Get().FindFilesRecursive(ProfileFileNames, *FLauncherProfile::GetProfileFolder(Pass == 1), TEXT("*.ulp2"), true, false);

		for (TArray<FString>::TConstIterator It(ProfileFileNames); It; ++It)
		{
			FString ProfileFilePath = *It;
			ILauncherProfilePtr LoadedProfile = LoadJSONProfile(*ProfileFilePath);

			if (LoadedProfile.IsValid())
			{
				if (Pass == 1)
				{
					LoadedProfile->SetNotForLicensees();
				}
				AddProfile(LoadedProfile.ToSharedRef());
			}
			else
			{
				UE_LOGF(LogLauncherProfile, Warning, "could not load %ls - ignoring", *ProfileFilePath);
			}
		}
	}
}


ILauncherDeviceGroupPtr FLauncherProfileManager::ParseDeviceGroup( const FString& GroupString )
{
	TSharedPtr<FLauncherDeviceGroup> Result;

	FString GroupIdString;

	if (FParse::Value(*GroupString, TEXT("Id="), GroupIdString))
	{
		FGuid GroupId;

		if (!FGuid::Parse(GroupIdString, GroupId))
		{
			GroupId = FGuid::NewGuid();
		}

		FString GroupName;
		FParse::Value(*GroupString, TEXT("Name="), GroupName);

		FString DevicesString;
		FParse::Value(*GroupString, TEXT("Devices="), DevicesString);

		Result = MakeShareable(new FLauncherDeviceGroup(GroupId, GroupName));

		TArray<FString> DeviceList;
		DevicesString.ParseIntoArray(DeviceList, TEXT(", "), true);

		for (int32 Index = 0; Index < DeviceList.Num(); ++Index)
		{
			Result->AddDevice(DeviceList[Index]);
		}
	}

	return Result;
}


void FLauncherProfileManager::SaveDeviceGroups( bool bTrimUnused )
{
	if (GConfig != nullptr)
	{
		TSet<FGuid> ActiveDeviceGroupIds;
		for ( const ILauncherProfilePtr& LauncherProfile : SavedProfiles )
		{
			for (const ILauncherProfileBuildCookRunRef& BuildCookRun : LauncherProfile->GetBuildCookRunCommands())
			{
				const ILauncherDeviceGroupPtr& Group = BuildCookRun->GetDeployedDeviceGroup(false);
				if (Group.IsValid())
				{
					ActiveDeviceGroupIds.Add( Group->GetId() );
				}
			}
		}


		GConfig->EmptySection(TEXT("Launcher.DeviceGroups"), ConfigFileName);

		TArray<FString> DeviceGroupStrings;


		// create a string representation of all groups and their devices
		for (int32 GroupIndex = 0; GroupIndex < DeviceGroups.Num(); ++GroupIndex)
		{
			const ILauncherDeviceGroupPtr& Group = DeviceGroups[GroupIndex];
			if (bTrimUnused && !ActiveDeviceGroupIds.Contains(Group->GetId()))
			{
				continue; // trim out unused groups when saving
			}

			const TArray<FString>& Devices = Group->GetDeviceIDs();

			FString DeviceListString;

			for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
			{
				if (DeviceIndex > 0)
				{
					DeviceListString += ", ";
				}

				DeviceListString += Devices[DeviceIndex];
			}

			FString DeviceGroupString = FString::Printf(TEXT("(Id=\"%s\", Name=\"%s\", Devices=\"%s\")" ), *Group->GetId().ToString(), *Group->GetName(), *DeviceListString);

			DeviceGroupStrings.Add(DeviceGroupString);
		}

		// save configuration
		GConfig->SetArray(TEXT("Launcher.DeviceGroups"), TEXT("DeviceGroup"), DeviceGroupStrings, ConfigFileName);
		GConfig->Flush(false, ConfigFileName);
	}
}


void FLauncherProfileManager::SaveSimpleProfiles()
{
	for (TArray<ILauncherSimpleProfilePtr>::TIterator It(SimpleProfiles); It; ++It)
	{
		FString SimpleProfileFileName = FLauncherProfile::GetProfileFolder(false) / (*It)->GetDeviceName() + TEXT(".uslp");
		FString Text;
		TSharedRef< TJsonWriter<> > Writer = TJsonWriterFactory<>::Create(&Text);
		(*It)->Save(Writer.Get());
		Writer->Close();
		FFileHelper::SaveStringToFile(Text, *SimpleProfileFileName);
	}
}


void FLauncherProfileManager::SaveProfiles( )
{
	for (TArray<ILauncherProfilePtr>::TIterator It(SavedProfiles); It; ++It)
	{
		SaveJSONProfile((*It).ToSharedRef());
	}
}

FString FLauncherProfileManager::MakeBuildCookRunParamsForProjectCustomBuild( const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms ) const
{
	FString Result = FLauncherWorker::MakeBuildCookRunParamsForProjectCustomBuild(InProfile, InPlatforms);
	return Result;
}
