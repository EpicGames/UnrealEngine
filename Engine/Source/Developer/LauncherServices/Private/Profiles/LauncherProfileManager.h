// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfileManager.h"
#include "Misc/Paths.h"

/**
 * Implements a helper class that manages all profiles in the Launcher
 */
class FLauncherProfileManager
	: public TSharedFromThis<FLauncherProfileManager>
	, public ILauncherProfileManager
{
public:

	/** Default constructor. */
	FLauncherProfileManager();


public:

	/**
	 * Loads the profiles and device groups.
	 */
	void Load();

	//~ Begin ILauncherProfileManager Interface

	virtual void AddDeviceGroup( const ILauncherDeviceGroupRef& DeviceGroup ) override;

	virtual ILauncherDeviceGroupRef AddNewDeviceGroup( ) override;

	virtual ILauncherDeviceGroupRef CreateUnmanagedDeviceGroup() override;

	virtual ILauncherSimpleProfilePtr FindOrAddSimpleProfile(const FString& DeviceName) override;

	virtual ILauncherSimpleProfilePtr FindSimpleProfile(const FString& DeviceName) override;

	virtual ILauncherProfileRef AddNewProfile() override;

	virtual ILauncherProfileRef CreateUnsavedProfile(FString ProfileName) override;

	virtual void AddProfile( const ILauncherProfileRef& Profile ) override;

	virtual ILauncherProfilePtr FindProfile( const FString& ProfileName ) override;

	virtual const TArray<ILauncherDeviceGroupPtr>& GetAllDeviceGroups( ) const override;

	virtual const TArray<ILauncherProfilePtr>& GetAllProfiles( ) const override;

	virtual ILauncherDeviceGroupPtr GetDeviceGroup( const FGuid& GroupId ) const override;

	virtual ILauncherProfilePtr GetProfile( const FGuid& ProfileId ) const override;

	UE_DEPRECATED(5.8, "Please use LoadJSONProfile")
	virtual ILauncherProfilePtr LoadProfile( FArchive& Archive ) override;

	virtual ILauncherProfilePtr LoadJSONProfile(FString ProfileFile) override;

	virtual void LoadSettings() override;

	virtual FOnLauncherProfileManagerDeviceGroupAdded& OnDeviceGroupAdded( ) override
	{
		return DeviceGroupAddedDelegate;
	}

	virtual FOnLauncherProfileManagerDeviceGroupRemoved& OnDeviceGroupRemoved( ) override
	{
		return DeviceGroupRemovedDelegate;
	}

	virtual FOnLauncherProfileManagerProfileAdded& OnProfileAdded( ) override
	{
		return ProfileAddedDelegate;
	}

	virtual FOnLauncherProfileManagerProfileRemoved& OnProfileRemoved( ) override
	{
		return ProfileRemovedDelegate;
	}

	virtual FOnLauncherProfileManagerProjectChanged& OnProjectChanged( ) override
	{
		return ProjectChangedDelegate;
	}

	virtual FOnLauncherProfilePostProcessCommandLine& OnPostProcessLaunchCommandLine( ) override
	{
		return ProcessProcessLaunchCommandLine;
	}

	UE_DEPRECATED(5.8, "Use OnPostProcessUATCommandLine")
	virtual FOnLauncherProfilePostProcessAutomatedTestCommandLine& OnPostProcessAutomatedTestCommandLine( ) override
	{
		return PostProcessAutomatedTestCommandLine;
	}

	virtual FOnLauncherProfilePostProcessUATCommandLine& OnPostProcessUATCommandLine( ) override
	{
		return PostProcessUATCommandLine;
	}

	virtual void RemoveDeviceGroup( const ILauncherDeviceGroupRef& DeviceGroup ) override;

	virtual void SaveDeviceGroups( bool bTrimUnused = false ) override;

	virtual void RemoveSimpleProfile(const ILauncherSimpleProfileRef& SimpleProfile) override;

	virtual void RemoveProfile( const ILauncherProfileRef& Profile ) override;

	UE_DEPRECATED(5.8, "Please use SaveJSONProfile")
	virtual bool SaveProfile( const ILauncherProfileRef& Profile) override;

	virtual bool SaveJSONProfile(const ILauncherProfileRef& Profile) override;

	virtual void ChangeProfileName( const ILauncherProfileRef& Profile, FString Name) override;

	virtual void RegisterProfileWizard(const ILauncherProfileWizardPtr& ProfileWizard) override;

	virtual void UnregisterProfileWizard(const ILauncherProfileWizardPtr& ProfileWizard) override;

	virtual const TArray<ILauncherProfileWizardPtr>& GetProfileWizards() const override;

	virtual void SaveSettings( ) override;

	virtual FString GetProjectName() const override;

	virtual FString GetProjectBasePath() const override;

	virtual FString GetProjectPath() const override;

	virtual void SetProjectPath(const FString& InProjectPath) override;

	virtual FString GetBuildTarget() const override;

	virtual void SetBuildTarget( const FString& InBuildTarget ) override;

	UE_DEPRECATED(5.8, "this was only used for the legacy project launcher")
	virtual const TArray<FString> GetAllExplicitBuildTargetNames() const override;

	virtual FString MakeBuildCookRunParamsForProjectCustomBuild(const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms) const override;

	//~ End ILauncherProfileManager Interface

protected:

	/**
	 * Loads all the device groups from a config file
	 */
	void LoadDeviceGroups( );

	/**
	 * Load all profiles from disk.
	 */
	void LoadProfiles( );

	/**
	 * Create a new device group from the specified string value.
	 *
	 * @param GroupString - The string to parse.
	 *
	 * @return The new device group object.
	 */
	ILauncherDeviceGroupPtr ParseDeviceGroup( const FString& GroupString );

	/*
	* Saves all simple profiles to disk.
	*/
	void SaveSimpleProfiles();

	/**
	 * Saves all profiles to disk.
	 */
	void SaveProfiles();

protected:

	/**
	* Gets the folder in which old profile files were stored.
	*
	* @return The folder path.
	*/
	static FString GetLegacyProfileFolder()
	{
		return FPaths::EngineSavedDir() / TEXT("Launcher");
	}

private:

	// Holds the collection of device groups.
	TArray<ILauncherDeviceGroupPtr> DeviceGroups;

	// Holds the collection of simple launcher profiles.
	TArray<ILauncherSimpleProfilePtr> SimpleProfiles;

	// Holds the collection of launcher profiles.
	TArray<ILauncherProfilePtr> SavedProfiles;

	// Holds the collection of launcher profiles.
	TArray<ILauncherProfilePtr> AllProfiles;

	// Holds the currently selected project path
	FString ProjectPath;

	// Holds the currently selected build target
	FString BuildTarget;
	
	// Holds all registered profile wizards
	TArray<ILauncherProfileWizardPtr> ProfileWizards;

	// Holds the config file name to save to
	FString ConfigFileName;

private:

	// Holds a delegate to be invoked when a device group was added.
	FOnLauncherProfileManagerDeviceGroupAdded DeviceGroupAddedDelegate;

	// Holds a delegate to be invoked when a device group was removed.
	FOnLauncherProfileManagerDeviceGroupRemoved DeviceGroupRemovedDelegate;

	// Holds a delegate to be invoked when a launcher profile was added.
	FOnLauncherProfileManagerProfileAdded ProfileAddedDelegate;

	// Holds a delegate to be invoked when a launcher profile was removed.
	FOnLauncherProfileManagerProfileRemoved ProfileRemovedDelegate;
	
	// Holds a delegate to be invoked when the project changes.
	FOnLauncherProfileManagerProjectChanged ProjectChangedDelegate;	

	// Holds a delegate to be invoked to modify the command line before launching
	FOnLauncherProfilePostProcessCommandLine ProcessProcessLaunchCommandLine;

	// Holds a delegate to be involked to modify the command line for an automated test before launching
	FOnLauncherProfilePostProcessAutomatedTestCommandLine PostProcessAutomatedTestCommandLine;

	// Holds a delegate to be involked to modify the command line for an extra UAT command before launching
	FOnLauncherProfilePostProcessUATCommandLine PostProcessUATCommandLine;
};
