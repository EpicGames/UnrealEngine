// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum ELauncherVersion
{
	LAUNCHERSERVICES_MINPROFILEVERSION=10,
	LAUNCHERSERVICES_ADDEDINCREMENTALDEPLOYVERSION=11,
	LAUNCHERSERVICES_ADDEDPATCHSOURCECONTENTPATH=12,
	LAUNCHERSERVICES_ADDEDRELEASEVERSION=13,
	LAUNCHERSERVICES_REMOVEDPATCHSOURCECONTENTPATH=14,
	LAUNCHERSERVICES_ADDEDDLCINCLUDEENGINECONTENT=15,
	LAUNCHERSERVICES_ADDEDGENERATECHUNKS=16,
	LAUNCHERSERVICES_ADDEDNUMCOOKERSTOSPAWN=17,
	LAUNCHERSERVICES_ADDEDSKIPCOOKINGEDITORCONTENT=18,
	LAUNCHERSERVICES_ADDEDDEFAULTDEPLOYPLATFORM=19,
	LAUNCHERSERVICES_FIXCOMPRESSIONSERIALIZE=20,
	LAUNCHERSERVICES_SHAREABLEPROJECTPATHS = 21,
	LAUNCHERSERVICES_FILEFORMATCHANGE = 22,
	LAUNCHERSERVICES_ADDARCHIVE = 23,
	LAUNCHERSERVICES_ADDEDENCRYPTINIFILES = 24,
	LAUNCHERSERVICES_ADDEDMULTILEVELPATCHING = 25,
	LAUNCHERSERVICES_ADDEDADDITIONALCOMMANDLINE = 26,
	LAUNCHERSERVICES_ADDEDINCLUDEPREREQUISITES = 27,
	LAUNCHERSERVICES_ADDEDBUILDMODE = 28,
	LAUNCHERSERVICES_ADDEDUSEIOSTORE = 29,
	LAUNCHERSERVICES_ADDEDMAKEBINARYCONFIG = 30,
	LAUNCHERSERVICES_ADDEDREFERENCECONTAINERS = 31,
	LAUNCHERSERVICES_REMOVEDNUMCOOKERSTOSPAWN = 32,
	LAUNCHERSERVICES_ADDEDORIGINALRELEASEVERSION = 33,
	LAUNCHERSERVICES_ADDBUILDTARGETNAME = 34,
	LAUNCHERSERVICES_ADDEDRETAINSTAGEDDIRECTORY = 35,
	LAUNCHERSERVICES_REMOVEDRETAINSTAGEDDIRECTORY = 36,
	LAUNCHERSERVICES_ADDZENOPTIONS = 37,
	LAUNCHERSERVICES_ADDBUILDARCHITECTURE = 38,
	LAUNCHERSERVICES_ADDCUSTOMPROPERTIES = 39,
	LAUNCHERSERVICES_INCREMENTALCOOKMODE = 40,
	LAUNCHERSERVICES_AUTOMATEDTESTS = 41,
	LAUNCHERSERVICES_FASTITERATE = 42,
	LAUNCHERSERVICES_REMOVEDINCREMENTALCOOKMODE = 43,
	LAUNCHERSERVICES_ADDMULTIPLEBUILDTARGETS = 44,
	LAUNCHERSERVICES_ADDUATCOMMANDS = 45,
	LAUNCHERSERVICES_INNERBUILDCOOKRUN = 46,
	LAUNCHERSERVICES_ADDZENSTREAMINGOPTIONS = 47,
	LAUNCHERSERVICES_ADDPRESTAGED = 48,
	//ADD NEW STUFF HERE


	LAUNCHERSERVICES_FINALPLUSONE,
	LAUNCHERSERVICES_FINAL = LAUNCHERSERVICES_FINALPLUSONE-1,
};
