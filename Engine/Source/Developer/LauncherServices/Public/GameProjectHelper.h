// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "DesktopPlatformModule.h"

/**
 * Implements a helper class for finding project specific information.
 */
class FGameProjectHelper
{
public:

	/**
	 * Gets a list of available build configurations for the specified game name.
	 *
	 * @param GameName - The name of the game (i.e. Example or UDK).
	 *
	 * @return A list of build configuration names.
	 */
	static TArray<FString> GetAvailableConfigurations( const FString& GameName )
	{
		TArray<FString> Result;

		// @todo gmp: discover the executables that are actually available
		Result.Add(TEXT("Debug"));
		Result.Add(TEXT("Development"));
		Result.Add(TEXT("Shipping"));
		Result.Add(TEXT("Test"));

		return Result;
	}

	/**
	 * Gets a list of available game names.
	 *
	 * @return Game names.
	 */
	static TArray<FString> GetAvailableGames( )
	{
		// @todo Rocket: Projects: Some incorrect assumptions about game project directories here

		TArray<FString> Result;
		TArray<FString> DirectoryNames;

		FString SearchPath = FPaths::RootDir() / FString(TEXT("*"));

		IFileManager::Get().FindFiles(DirectoryNames, *SearchPath, false, true);

		for (int32 FileIndex = 0; FileIndex < DirectoryNames.Num(); ++FileIndex)
		{
			FString GameName = DirectoryNames[FileIndex];

			if (IsGameAvailable(GameName))
			{
				Result.AddUnique(GameName);
			}
		}

		return Result;
	}

	/**
	 * Gets the list of available maps for the specified game.
	 *
	 * @param GameName - The name of the game (i.e. Example or UDK).
	 * @param IncludeEngineMaps - Whether maps in the Engine folder should be included.
	 * @param Sorted - Whether the list of maps should be sorted alphabetically.
	 *
	 * @return A list of available map names.
	 */
	static TArray<FString> GetAvailableMaps( FString GameName, bool IncludeEngineMaps, bool Sorted )
	{
		TArray<FString> Result;
		TArray<FString> EnginemapNames;
		TArray<FString> ProjectMapNames;

		const FString WildCard = FString::Printf(TEXT("*%s"), *FPackageName::GetMapPackageExtension());

		// Scan all Content folder, because not all projects follow Content/Maps convention
		IFileManager::Get().FindFilesRecursive(ProjectMapNames, *FPaths::Combine(*FPaths::RootDir(), *GameName, TEXT("Content")), *WildCard, true, false);

		// didn't find any, let's check the base GameName just in case it is a full path
		if (ProjectMapNames.Num() == 0)
		{
			IFileManager::Get().FindFilesRecursive(ProjectMapNames, *FPaths::Combine(*GameName, TEXT("Content")), *WildCard, true, false);
		}

		for (int32 i = 0; i < ProjectMapNames.Num(); i++)
		{
			Result.Add(FPaths::GetBaseFilename(ProjectMapNames[i]));
		}

		if (IncludeEngineMaps)
		{
			IFileManager::Get().FindFilesRecursive( EnginemapNames, *FPaths::Combine(*FPaths::RootDir(), TEXT("Engine"), TEXT("Content"), TEXT("Maps")), *WildCard, true, false);

			for (int32 i = 0; i < EnginemapNames.Num(); i++)
			{
				Result.Add(FPaths::GetBaseFilename(EnginemapNames[i]));
			}
		}

		if (Sorted)
		{
			Result.Sort();
		}

		return Result;
	}

	/**
	 * Checks whether the specified game is available.
	 *
	 * @param GameName - The name of the game (i.e. ExampleGame or UDKGame).
	 *
	 * @return true if the game is available, false otherwise.
	 */
	static bool IsGameAvailable( FString GameName )
	{
		const bool bIsEngineDirectory = (GameName == TEXT("Engine")); // The engine directory is not a game.
		if (!bIsEngineDirectory && IFileManager::Get().DirectoryExists(*(FPaths::RootDir() / GameName / TEXT("Config"))))
		{
			return true;
		}

		return false;
	}

	/**
	 * Returns the list of build targets that must be explicitly specified for the given project.
	 * @param InProjectFilePath - the project to use
	 * @param InFilterTargetTypes - if specified, the returned list will be restricted to the given target types.
	 */
	static TArray<FString> GetExplicitBuildTargetsForProject( const FString& InProjectFilePath, const TSet<EBuildTargetType>* InFilterTargetTypes = nullptr )
	{
		TArray<FString> ExplicitBuildTargetNames;

		if (!InProjectFilePath.IsEmpty())
		{
			// ignore the filter if it's empty
			if (InFilterTargetTypes != nullptr && InFilterTargetTypes->Num() == 0)
			{
				InFilterTargetTypes = nullptr;
			}

			// collect the build targets for the given project
			TMap<EBuildTargetType,TArray<FString>> BuildTargetNamesByType;
			const TArray<FTargetInfo>& Targets = FDesktopPlatformModule::Get()->GetTargetsForProject(InProjectFilePath);
			for (const FTargetInfo& Target : Targets)
			{
				if (Target.Type != EBuildTargetType::Game && Target.Type != EBuildTargetType::Client && Target.Type != EBuildTargetType::Server)
				{
					continue;
				}
				
				if (InFilterTargetTypes != nullptr && !InFilterTargetTypes->Contains(Target.Type) )
				{
					continue;
				}

				BuildTargetNamesByType.FindOrAdd(Target.Type).Add(Target.Name);
			}

			// create a list of build targets that need to be specified explicitly (because there is more than one of the same type)
			for (auto& Itr : BuildTargetNamesByType)
			{
				if (Itr.Value.Num() > 1)
				{
					ExplicitBuildTargetNames.Append(Itr.Value);
				}
			}
		}

		return MoveTemp(ExplicitBuildTargetNames);
	}
};
