// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherWorker.h"

class ILauncher;
class ITargetDeviceProxyManager;


/** Type definition for shared pointers to instances of ILauncher. */
typedef TSharedPtr<class ILauncher> ILauncherPtr;

/** Type definition for shared references to instances of ILauncher. */
typedef TSharedRef<class ILauncher> ILauncherRef;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLauncherWorkerStarted, ILauncherWorkerPtr, ILauncherProfileRef);


/**
 * Interface for game launchers.
 */
class ILauncher
{
public:

	/**
	 * Launches the specified profile.
	 *
	 * @param DeviceProxyManager The target device proxy manager to use.
	 * @param Profile The profile to launch.
	 * @return The worker thread, or nullptr if not launched.
	 */
	virtual ILauncherWorkerPtr Launch(const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager, const ILauncherProfileRef& Profile) = 0;

public:

	/** Virtual destructor. */
	virtual ~ILauncher( ) = default;

	FOnLauncherWorkerStarted FLauncherWorkerStartedDelegate;
};
