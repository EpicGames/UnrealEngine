// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

class ILauncherDeviceGroup;

/** Type definition for shared pointers to instances of ILauncherDeviceGroup. */
typedef TSharedPtr<class ILauncherDeviceGroup> ILauncherDeviceGroupPtr;

/** Type definition for shared references to instances of ILauncherDeviceGroup. */
typedef TSharedRef<class ILauncherDeviceGroup> ILauncherDeviceGroupRef;


/**
 * Delegate type for adding devices.
 *
 * The first parameter is the device group that invoked the delegate.
 * The second parameter is the identifier of the device that was removed.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLauncherDeviceGroupDeviceAdded, const ILauncherDeviceGroupRef&, const FString&)

/**
 * Delegate type for removing devices.
 *
 * The first parameter is the device group that invoked the delegate.
 * The second parameter is the identifier of the device that was removed.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLauncherDeviceGroupDeviceRemoved, const ILauncherDeviceGroupRef&, const FString&)


/**
 * Interface for Launcher device groups.
 */
class ILauncherDeviceGroup
{
public:

	/**
	 * Adds a device to the group.
	 *
	 * @param DeviceID The identifier of the device to add.
	 */
	virtual void AddDevice(const FString& DeviceID) = 0;
	
	/**
	 * Get a list of identifiers for devices associated with this device group.
	 *
	 * @return The list of device identifiers.
	 */
	virtual const TArray<FString>& GetDeviceIDs() = 0;

	/**
	 * Gets the unique identifier of the device group.
	 *
	 * @return The device group identifier.
	 */
	virtual FGuid GetId() const = 0;

	/**
	 * Gets the human readable name of the device group.
	 *
	 * @return The name of the device group.
	 */
	virtual const FString& GetName() const = 0;

	/**
	 * Get a list of devices associated with this device group.
	 *
	 * @return The number of devices the device group maintains.
	 */
	virtual int32 GetNumDevices() const = 0;

	/**
	 * Removes a device from the group.
	 *
	 * @param DeviceID The identifier of the device to remove.
	 */
	virtual void RemoveDevice(const FString& DeviceID) = 0;

	/**
	* Removes all devices from the group.
	*	
	*/
	virtual void RemoveAllDevices() = 0;

	/**
	 * Sets the name of the device group.
	 *
	 * @param NewName The new name.
	 */
	virtual void SetName(const FString& NewName) = 0;

public:

	/**
	 * Returns a delegate that is invoked when device was added to this group.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherDeviceGroupDeviceAdded& OnDeviceAdded() = 0;

	/**
	 * Returns a delegate that is invoked when device was removed from this group.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherDeviceGroupDeviceRemoved& OnDeviceRemoved() = 0;

public:

	/** Virtual destructor. */
	virtual ~ILauncherDeviceGroup() = default;
};
