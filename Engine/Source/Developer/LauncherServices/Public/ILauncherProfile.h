// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Containers/Map.h"
#include "ILauncherDeviceGroup.h"
#include "ILauncherProfileLaunchRole.h"
#include "ILauncherProfileAutomatedTest.h"
#include "ILauncherProfileBuildCookRun.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLauncherProfile, Log, All);

class Error;
class FJsonObject;
class ILauncherProfile;
class ILauncherSimpleProfile;


/** Type definition for shared pointers to instances of ILauncherProfile. */
typedef TSharedPtr<class ILauncherSimpleProfile> ILauncherSimpleProfilePtr;

/** Type definition for shared references to instances of ILauncherProfile. */
typedef TSharedRef<class ILauncherSimpleProfile> ILauncherSimpleProfileRef;

/**
* Interface for simple launcher profile.
*/
class ILauncherSimpleProfile
{
public:

	/**
	 * Gets the device name this profile is for.
	 *
	 * @return Device Name.
	 */
	virtual const FString& GetDeviceName() const = 0;

	/**
	 * Gets the device variant to use when deploying and launching.
	 *
	 * @return Device Variant name.
	 * @see SetDeviceVariant
	 */
	virtual FName GetDeviceVariant() const = 0;

	/**
	 * Gets the name of the build configuration.
	 *
	 * @return Build configuration name.
	 * @see SetBuildConfigurationName
	 */
	virtual EBuildConfiguration GetBuildConfiguration() const = 0;

	/**
	 * Gets the selected cook mode.
	 *
	 * @return Cook mode.
	 */
	virtual ELauncherProfileCookModes::Type GetCookMode() const = 0;

	/**
	* Loads the simple profile from the specified file.
	*
	* @return true if the profile was loaded, false otherwise.
	*/
	virtual bool Load(const FJsonObject& Object) = 0;

	/**
	* Saves the simple profile to the specified file.
	*
	* @return true if the profile was saved, false otherwise.
	*/
	virtual void Save(TJsonWriter<>& Writer) = 0;

	/**
	 * Updates the device name.
	 *
	 * @param InDeviceName The new device name.
	 */
	virtual void SetDeviceName(const FString& InDeviceName) = 0;

	/**
	 * Sets the device variant.
	 *
	 * @param InVariant The variant to set.
	 * @see GetDeviceVariant
	 */
	virtual void SetDeviceVariant(FName InVariant) = 0;

	/**
	 * Sets the build configuration.
	 *
	 * @param InConfiguration The build configuration name to set.
	 * @see GetBuildConfigurationName
	 */
	virtual void SetBuildConfiguration(EBuildConfiguration InConfiguration) = 0;

	/**
	 * Sets the cook mode.
	 *
	 * @param InMode The cook mode.
	 * @see GetCookMode
	 */
	virtual void SetCookMode(ELauncherProfileCookModes::Type InMode) = 0;

	/**
	 * Serializes the simple profile from or into the specified archive.
	 *
	 * @param Archive The archive to serialize from or into.
	 * @return true if the profile was serialized, false otherwise.
	 */
	UE_DEPRECATED(5.8, "this will be removed. there is no alternative implementation : ILauncherSimpleProfile is designed to be transient")
	virtual bool Serialize(FArchive& Archive) = 0;

	/** Sets all profile settings to their defaults. */
	virtual void SetDefaults() = 0;

public:

	/**
	* Virtual destructor.
	*/
	virtual ~ILauncherSimpleProfile() = default;
};


/** Type definition for shared pointers to instances of ILauncherProfile. */
typedef TSharedPtr<class ILauncherProfile> ILauncherProfilePtr;

/** Type definition for shared references to instances of ILauncherProfile. */
typedef TSharedRef<class ILauncherProfile> ILauncherProfileRef;


/**
 * Delegate type for changing the device group to deploy to.
 *
 * The first parameter is the selected device group (or NULL if the selection was cleared).
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileDeployedDeviceGroupChanged, const ILauncherDeviceGroupPtr&)

/** Delegate type for a change in project */
DECLARE_MULTICAST_DELEGATE(FOnProfileProjectChanged);

/** Delegate type for a change in build target options */
DECLARE_MULTICAST_DELEGATE(FOnProfileBuildTargetOptionsChanged);


/** Delegate type for a UAT command */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileUATCommandChanged, const ILauncherProfileUATCommandRef&)



/** Delegate type for detecting if cook is finished
 *	Used when cooking from the editor.  Specific cook task will wait for the cook to be finished by the editor
 */
DECLARE_DELEGATE_RetVal(bool, FIsCookFinishedDelegate);

/**
 * Delegate type used to callback if the cook has been canceled
 * only used for cook by the book in editor
 */
DECLARE_DELEGATE(FCookCanceledDelegate);

/**
 * Interface for launcher profile.
 */
class ILauncherProfile
{
public:

	/**
	 * Gets the unique identifier of the profile.
	 *
	 * @return The profile identifier.
	 */
	virtual FGuid GetId( ) const = 0;

	/**
	* Gets the file name for serialization.
	*
	* @return The file name.
	*/
	virtual FString GetFileName( ) const = 0;

	/**
	* Gets the full file path for serialization.
	*
	* @return The file path.
	*/
	virtual FString GetFilePath() const = 0;

	/**
	 * Gets the human readable name of the profile.
	 *
	 * @return The profile name.
	 */
	virtual FString GetName( ) const = 0;

	/**
	 * Gets the human readable description of the profile.
	 *
	 * @return The profile description.
	 */
	virtual FString GetDescription() const = 0;

	/**
	 * Checks whether the last validation yielded any error.
	 *
   	 * @param UATCommand Optional UAT command associated with this error
	 * @return true if the any error is present, false otherwise.
	 */
	virtual bool HasValidationError(ILauncherProfileUATCommandPtr UATCommand = nullptr) const = 0;

	/**
	 * Checks whether the last validation yielded the specified error.
	 *
	 * @param Error The validation error to check for.
   	 * @param UATCommand Optional UAT command associated with this error
	 * @return true if the error is present, false otherwise.
	 */
	virtual bool HasValidationError( ELauncherProfileValidationErrors::Type Error, ILauncherProfileUATCommandPtr UATCommand = nullptr ) const = 0;




	/**
	 * Add a custom error. This will prevent launching
	 * 
	 * @param UniqueId Unique identifier for this error
	 * @param Text Localized string describing this error
 	 * @param UATCommand Optional UAT command associated with this error
	 */
	virtual void AddCustomError( const FString& UniqueId, const FText& Text, ILauncherProfileUATCommandPtr UATCommand = nullptr ) = 0;


	/**
	 * Checks whether this profile has the given custom error
	 *
	 * @param UniqueId Unique identifier for this error
	 */
	virtual bool HasCustomError( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand = nullptr ) const = 0;

	/**
	 * Gets the localized string representation of the given error
	 * 
	 * @param UniqueId
	 * @returns Localized string describing this error, or empty text if we don't have it
	 */
	virtual FText GetCustomErrorText( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand = nullptr ) const = 0;

	/**
	 * Gets all custom errors in this profile
	 * 
	 * @returns UniqueIds for all custom errors in this profile
	 */
	virtual TArray<FString> GetAllCustomErrors(ILauncherProfileUATCommandPtr UATCommand = nullptr) const = 0;
	


	/**
	 * Add a custom warning. This does not prevent launching.
	 * 
	 * @param UniqueId Unique identifier for this warning
	 * @param Text Localized string describing this warning
 	 * @param UATCommand Optional UAT command associated with this warning
	 */
	virtual void AddCustomWarning( const FString& UniqueId, const FText& Text, ILauncherProfileUATCommandPtr UATCommand = nullptr ) = 0;

	/**
	 * Checks whether this profile has the given custom warning
	 *
	 * @param UniqueId Unique identifier for this warning
	 */
	virtual bool HasCustomWarning( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand = nullptr ) const = 0;

	/**
	 * Gets the localized string representation of the given warning
	 * 
	 * @param UniqueId
	 * @returns Localized string describing this warning, or empty text if we don't have it
	 */
	virtual FText GetCustomWarningText( const FString& UniqueId, ILauncherProfileUATCommandPtr UATCommand = nullptr ) const = 0;

	/**
	 * Gets all custom warnings in this profile
	 * 
	 * @returns UniqueIds for all custom warnings in this profile
	 */
	virtual TArray<FString> GetAllCustomWarnings(ILauncherProfileUATCommandPtr UATCommand = nullptr) const = 0;


	/**
	 * Refreshes the custom warnings and errors. Resets the lists and triggers the OnValidation callback.
	 */
	virtual void RefreshCustomWarningsAndErrors(ILauncherProfileUATCommandPtr UATCommand = nullptr) = 0;



	/**
	 * Gets the invalid platform, this is only valid when there is a platform centric validation error.
	 *
	 * @return string specifying the invalid platform.
	 */
	virtual FString GetInvalidPlatform(ILauncherProfileUATCommandPtr UATCommand = nullptr) const = 0;

	/**
	 * Checks whether devices of the specified platform can be deployed to.
	 *
	 * Whether a platform is deployable depends on the current profile settings.
	 * The right combination of build, cook and package settings must be present.
	 *
	 * @param PlatformName The name of the platform to deploy.
	 * @return true if the platform is deployable, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsDeployablePlatform( const FString& PlatformName ) = 0;

	/**
	 * Checks whether this profile is valid to use when running a game instance.
	 *
	 * @return Whether the profile is valid or not.
	 */
	virtual bool IsValidForLaunch( ) = 0;

	/**
	* Loads the profile from a JSON file
	*/
	virtual bool Load(const FJsonObject& Object) = 0;

	/**
	 * Serializes the profile from or into the specified archive.
	 *
	 * @param Archive The archive to serialize from or into.
	 * @return true if the profile was serialized, false otherwise.
	 */
	UE_DEPRECATED(5.8, "please use the Json Save/Load API instead")
	virtual bool Serialize( FArchive& Archive ) = 0;

	/**
	 * Saves the profile into a JSON file
	 */
	virtual void Save(TJsonWriter<>& Writer) = 0;

	/** Sets all profile settings to their defaults. */
	virtual void SetDefaults( ) = 0;

	/** 
	 * Ensures this profile has an Id, if it doesn't already have one
	 */
	virtual void AssignId( bool bOverrideExisting = false) = 0;

	/**
	 * Updates the name of the profile.
	 *
	 * @param NewName The new name of the profile.
	 */
	virtual void SetName( const FString& NewName ) = 0;

	/**
	 * Updates the description of the profile.
	 *
	 * @param NewDescription The new description of the profile.
	 */
	virtual void SetDescription(const FString& NewDescription) = 0;

	/**
	* Changes the save location to an internal project path.
	*	
	*/
	virtual void SetNotForLicensees() = 0;

	/**
	 * Returns the cook delegate which can be used to query if the cook is finished.
	 *
	 * Used by cook by the book in the editor.
	 */
	virtual FIsCookFinishedDelegate& OnIsCookFinished() = 0;

	/**
	 * Returns the cook delegate which should be called if the cook is canceled
	 *  Used by cook by the book in the editor
	 */
	virtual FCookCanceledDelegate& OnCookCanceled() = 0;

public:

	/**
	 * Gets the name of the build configuration.
	 *
	 * @return Build configuration name.
	 * @see SetBuildConfigurationName
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual EBuildConfiguration GetBuildConfiguration( ) const = 0;

	/**
	 * Checks whether the profile specifies a build target.
	 *
	 * @return true if the profile specifies a build target.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool HasBuildTargetSpecified() const = 0;

	/**
	 * Gets the build target.
	 *
	 * @Return Target name to build/run (it would match a .Target.cs file)
	 * @see SetBuildTarget
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version and GetBuildTargets")
	virtual FString GetBuildTarget() const = 0;

	/**
	 * Gets the build configuration name of the cooker.
	 *
	 * @return Cook configuration name.
	 * @see SetCookConfigurationName
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual EBuildConfiguration GetCookConfiguration( ) const = 0;

	/**
	 * Gets the selected cook mode.
	 *
	 * @return Cook mode.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ELauncherProfileCookModes::Type GetCookMode( ) const = 0;

	/**
	 * Gets the cooker command line options.
	 *
	 * @return Cook options string.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const FString& GetCookOptions( ) const = 0;


	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const bool GetSkipCookingEditorContent() const = 0; 

	/**
	 * Skip editor content while cooking, 
	 * This will strip editor content from final builds
	 * 
	 * @param InSkipCookingEditorContent
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetSkipCookingEditorContent(const bool InSkipCookingEditorContent) = 0;


	/**
	 * Gets the list of cooked culture.
	 *
	 * @return Collection of culture names.
	 * @see AddCookedCulture, ClearCookedCultures, RemoveCookedCulture
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<FString>& GetCookedCultures( ) const = 0;

	/**
	 * Gets the list of cooked maps.
	 *
	 * @return Collection of map names.
	 *
	 * @see AddCookedMap, ClearCookedMaps, RemoveCookedMap
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<FString>& GetCookedMaps( ) const = 0;

	/**
	 * Gets the names of the platforms to build for.
	 *
	 * @return Read-only collection of platform names.
	 *
	 * @see AddCookedPlatform, ClearCookedPlatforms, RemoveCookedPlatform
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<FString>& GetCookedPlatforms( ) const = 0;

	/**
	 * Gets the default launch role.
	 *
	 * @return A reference to the default launch role.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const ILauncherProfileLaunchRoleRef& GetDefaultLaunchRole( ) const = 0;

	/**
	 * Gets the device group to deploy to.
	 *
	 * @return The device group, or NULL if none was configured.
	 * @see SetDeployedDeviceGroup
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ILauncherDeviceGroupPtr GetDeployedDeviceGroup( ) = 0;

	/**
	* Gets the default platforms to deploy if no specific devices were selected.
	*	
	*/
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const FName GetDefaultDeployPlatform() const = 0;

	/**
	 * Gets the deployment mode.
	 *
	 * @return The deployment mode.
	 * @see SetDeploymentMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ELauncherProfileDeploymentModes::Type GetDeploymentMode( ) const = 0;

    /**
     * Gets the close mode for the cook on the fly server
     *
     * @return The close mode.
     * @see SetForceClose
     */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
    virtual bool GetForceClose() const = 0;
    
	/**
	 * Gets the launch mode.
	 *
	 * @return The launch mode.
	 * @see SetLaunchMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ELauncherProfileLaunchModes::Type GetLaunchMode( ) const = 0;

	/**
	 * Gets the profile's collection of launch roles.
	 *
	 * @return A read-only collection of launch roles.
	 * @see CreateLaunchRole, RemoveLaunchRole
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<ILauncherProfileLaunchRolePtr>& GetLaunchRoles( ) const = 0;

	/**
	 * Gets the launch roles assigned to the specified device.
	 *
	 * @param DeviceId The identifier of the device.
	 * @param OutRoles Will hold the assigned roles, if any.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const int32 GetLaunchRolesFor( const FString& DeviceId, TArray<ILauncherProfileLaunchRolePtr>& OutRoles ) = 0;

	/**
	 * Gets the packaging mode.
	 *
	 * @return The packaging mode.
	 * @see SetPackagingMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ELauncherProfilePackagingModes::Type GetPackagingMode( ) const = 0;

	/**
	 * Gets the packaging directory.
	 *
	 * @return The packaging directory.
	 * @see SetPackageDirectory
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetPackageDirectory( ) const = 0;

	/**
	 * Whether to archive build
	 *
	 * @see SetArchive
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsArchiving( ) const = 0;

	/**
	 * Gets the archive directory.
	 *
	 * @return The archive directory.
	 * @see SetArchiveDirectory
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetArchiveDirectory( ) const = 0;

	/**
	 * Checks whether the profile specifies a project.
	 * Not specifying a project means that it can be used for any project.
	 *
	 * @return true if the profile specifies a project.
	 */
	virtual bool HasProjectSpecified() const = 0;

	/**
	 * Gets the name of the Unreal project to use.
	 */
	virtual FString GetProjectName( ) const = 0;

	/**
	 * Gets the base project path for the project (e.g. Samples/Showcases/MyShowcase)
	 */
	virtual FString GetProjectBasePath() const = 0;

	/**
	 * Gets the full path to the Unreal project to use.
	 *
	 * @return The path.
	 * @see SetProjectPath
	 */
	virtual FString GetProjectPath( ) const = 0;

	/**
	 * Gets the additional command line parameters that will be used when the app launches.
	 * These will be used by all launch roles
	 *
	 * @return The additional command line parameters
	 * @see SetAdditionalCommandLineParameters
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetAdditionalCommandLineParameters() const = 0;

	/**
	 * Gets the additional command line parameters that will be used for the given target type when the app launches.
	 *
 	 * @param	BuildTargetType	The type of build target to query
	 * @return The additional command line parameters
	 * @see SetAdditionalTargetCommandLineParameters
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetAdditionalTargetCommandLineParameters( EBuildTargetType BuildTargetType ) const = 0;

    /**
     * Gets the timeout time for the cook on the fly server.
     *
     * @return The timeout time.
     * @see SetTimeout
     */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
    virtual uint32 GetTimeout() const = 0;
	
	/**
	 * Are we going to generate a patch (Source content patch needs to be specified)
	 * @Seealso GetPatchSourceContentPath	 * @Seealso SetPatchSourceContentPath
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsGeneratingPatch() const = 0;
	
	/**
	 * Are we going to generate a new patch tier
	 */
	virtual bool ShouldAddPatchLevel() const = 0;

	/**
	 * Should we stage the pak files from the base release version this patch is built on
	 */
	UE_DEPRECATED(5.6, "ShouldStageBaseReleasePaks is no longer used")
	virtual bool ShouldStageBaseReleasePaks() const = 0;

	/**
	 * Gets the current build mode.
	 *
	 * @return The current build mode.
	 * @see SetBuildMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ELauncherProfileBuildModes::Type GetBuildMode() const = 0;

	/**
	 * Determines whether the current profile requires building
	 *
	 * @return The current build mode.
	 * @see SetBuildMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool ShouldBuild() = 0;

	/**
	 * Checks if fast iterate is enabled.
	 *
	 * @return true if fast iterate is enabled, false otherwise.
	 * @see SetFastIterate
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsFastIterate() const = 0;
	
	/**
	 * Checks whether UAT should be built.
	 *
	 * @return true if building UAT, false otherwise.
	 * @see SetBuildGame
	 */
	virtual bool IsBuildingUAT() const = 0;

	/**
	 * Checks whether legacy incremental cooking is enabled.
	 *
	 * @return true if cooking incrementally, false otherwise.
	 * @see SetIncrementalCooking
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsCookingIncrementally( ) const = 0;

	/**
	 * Get the current incremental cook mode
	 * 
	 * @return incremental cook mode
	 * @see SetIncrementalCookMode
	 */ 
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version and IsCookingIncrementally again instead")
	virtual ELauncherProfileIncrementalCookMode::Type GetIncrementalCookMode() const = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsIterateSharedCookedBuild() const =0;

	/**
	 * Checks if compression is enabled
	 *
	 * @return true if compression is enabled
	 * @see SetCompressed
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsCompressed( ) const = 0;

	/**
	 * Checks if encrypting ini files is enabled
	 * 
	 * @return true if encrypting ini files is enabled
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsEncryptingIniFiles() const = 0;

	/**
	 * Checks if encrypting ini files is enabled
	 *
	 * @return true if encrypting ini files is enabled
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsForDistribution() const = 0;

	/**
	 * Checks whether unversioned cooking is enabled.
	 *
	 * @return true if cooking unversioned, false otherwise.
	 * @see SetUnversionedCooking
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsCookingUnversioned( ) const = 0;

	/**
	 * Checks whether incremental deployment is enabled.
	 *
	 * @return true if deploying incrementally, false otherwise.
	 * @see SetIncrementalDeploying
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsDeployingIncrementally( ) const = 0;

	/**
	 * Checks whether the file server's console window should be hidden.
	 *
	 * @return true if the file server should be hidden, false otherwise.
	 * @see SetHideFileServer
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsFileServerHidden( ) const = 0;

	/**
	 * Checks whether the file server is a streaming file server.
	 *
	 * @return true if the file server is streaming, false otherwise.
	 * @see SetStreamingFileServer
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsFileServerStreaming( ) const = 0;

	/**
	 * Checks whether packaging with UnrealPak is enabled.
	 *
	 * @return true if UnrealPak is used, false otherwise.
	 * @see SetPackageWithUnrealPak
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsPackingWithUnrealPak( ) const = 0;

	/**
	 * Checks whether to include an installer for prerequisites of packaged games, such as redistributable operating system components, on platforms that support it.
	 *
	 * @return true if prerequisites are to be included, false otherwise.
	 * @see SetIncludePrerequisites
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsIncludingPrerequisites() const = 0;

	/**
	 * Return whether packaging will generate chunk data.
	 *
	 * @return true if Chunks will be generated, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsGeneratingChunks() const = 0;
	
	/**
	 * Return whether packaging will use chunk data to generate http chunk install data.
	 *
	 * @return true if data will be generated, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsGenerateHttpChunkData() const = 0;
	
	/**
	 * Where generated http chunk install data will be stored.
	 *
	 * @return true if data will be generated, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetHttpChunkDataDirectory() const = 0;
	
	/**
	 * What name to tag the generated http chunk install data with.
	 *
	 * @return true if data will be generated, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetHttpChunkDataReleaseName() const = 0;

	/**
	 * Checks whether the profile's selected project supports Engine maps.
	 *
	 * @return true if Engine maps are supported, false otherwise.
	 */
	virtual bool SupportsEngineMaps( ) const = 0;

	/**
	 * Sets the path to the editor executable to use in UAT.
	 *
	 * @param EditorExe Path to the editor executable.
	 */
	virtual void SetEditorExe( const FString& EditorExe ) = 0;

	/**
	 * Gets the path to the editor executable.
	 *
	 * @return Path to the editor executable.
	 */
	virtual FString GetEditorExe( ) const = 0;

public:

	/**
	 * Adds a culture to cook (only used if cooking by the book).
	 *
	 * @param CultureName The name of the culture to cook.
	 *
	 * @see ClearCookedCultures, GetCookedCultures, RemoveCookedCulture
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void AddCookedCulture( const FString& CultureName ) = 0;

	/**
	 * Adds a map to cook (only used if cooking by the book).
	 *
	 * @param MapName The name of the map to cook.
	 *
	 * @see ClearCookedMaps, GetCookedMaps, RemoveCookedMap
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void AddCookedMap( const FString& MapName ) = 0;

	/**
	 * Adds a platform to cook (only used if cooking by the book).
	 *
	 * @param PlatformName The name of the platform to add.
	 *
	 * @see ClearCookedPlatforms, GetCookedPlatforms, RemoveCookedPlatform
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void AddCookedPlatform( const FString& PlatformName ) = 0;

	/**
	* Adds a platform to deploy (only used if a specific device is not specified).
	* Will deploy to the default device of the given platform, or the first device if none are
	* marked as 'default'.
	*
	* @param PlatformName The name of the platform to add.		
	*/
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDefaultDeployPlatform(const FName PlatformName) = 0;

	/**
	 * Removes all cooked cultures.
	 *
	 * @see AddCookedCulture, GetCookedCulture, RemoveCookedCulture
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void ClearCookedCultures( ) = 0;

	/**
	 * Removes all cooked maps.
	 *
	 * @see AddCookedMap, GetCookedMap, RemoveCookedMap
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void ClearCookedMaps( ) = 0;

	/**
	 * Removes all cooked platforms.
	 *
	 * @see AddCookedPlatform, GetCookedPlatforms, RemoveCookedPlatform
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void ClearCookedPlatforms( ) = 0;

	/**
	 * Creates a new launch role and adds it to the profile.
	 *
	 * @return The created role.
	 * @see GetLaunchRoles, RemoveLaunchRole
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual ILauncherProfileLaunchRolePtr CreateLaunchRole( ) = 0;

	/**
	 * Removes a cooked culture.
	 *
	 * @param CultureName The name of the culture to remove.
	 * @see AddCookedCulture, ClearCookedCultures, GetCookedCultures
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void RemoveCookedCulture( const FString& CultureName ) = 0;

	/**
	 * Removes a cooked map.
	 *
	 * @param MapName The name of the map to remove.
	 * @see AddCookedMap, ClearCookedMaps, GetCookedMaps
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void RemoveCookedMap( const FString& MapName ) = 0;

	/**
	 * Removes a platform from the cook list.
	 *
	 * @param PlatformName The name of the platform to remove.
	 * @see AddBuildPlatform, ClearCookedPlatforms, GetBuildPlatforms
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void RemoveCookedPlatform( const FString& PlatformName ) = 0;

	/**
	 * Removes the given launch role from the profile.
	 *
	 * @param Role The role to remove.
	 * @see CreateLaunchRole, GetLaunchRoles
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void RemoveLaunchRole( const ILauncherProfileLaunchRoleRef& Role ) = 0;

	/**
	 * Sets the current build mode
	 *
	 * @param Mode Whether the game should be built.
	 * @see GetBuildMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetBuildMode(ELauncherProfileBuildModes::Type Mode) = 0;

	/**
	 * Sets fast iterate.
	 *
	 * @param Enable fast iterate.
	 * @see IsFastIterate
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetFastIterate(bool Enable) = 0;

	/**
	 * Sets whether to build UAT.
	 *
	 * @param Build Whether UAT should be built.
	 * @see IsBuilding
	 */
	virtual void SetBuildUAT( bool Build ) = 0;

	/**
	 * Sets the build configuration.
	 *
	 * @param ConfigurationName The build configuration name to set.
	 * @see GetBuildConfigurationName
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetBuildConfiguration( EBuildConfiguration Configuration ) = 0;

	/**
	 * Sets whether this profile specfies a build target.
	 * 
	 * @param Specified Whether a build target is specified.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetBuildTargetSpecified(bool Specified) = 0;

	/** Notifies the profile that the fallback build target changed. */
	virtual void FallbackBuildTargetUpdated() = 0;

	/**
	 * Sets the build target.
	 *
	 * @param TargetName The target name to set (it would match a .Target.cs file)
	 * @see GetBuildTarget
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version and AddBuildTarget instead")
	virtual void SetBuildTarget( const FString& TargetName ) = 0;

	/**
	 * Sets the build configuration of the cooker.
	 *
	 * @param Configuration The cooker's build configuration to set.
	 * @see GetCookConfiguration
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCookConfiguration( EBuildConfiguration Configuration ) = 0;

	/**
	 * Sets the cook mode.
	 *
	 * @param Mode The cook mode.
	 * @see GetCookMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCookMode( ELauncherProfileCookModes::Type Mode ) = 0;

	/**
	 * Sets the cook options.
	 *
	 * @param Options The cook options.
	 * @see GetCookOptions
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCookOptions(const FString& Options) = 0;

	/**
	 * Sets whether to pack with UnrealPak.
	 *
	 * @param UseUnrealPak Whether UnrealPak should be used.
	 * @see IsPackingWithUnrealPak
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDeployWithUnrealPak( bool UseUnrealPak ) = 0;

	/**
	 * Set whether packaging will generate chunk data.
	 *
	 * @param true if Chunks should be generated, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetGenerateChunks(bool bGenerateChunks) = 0;
	/**
	 * Set whether packaging will use chunk data to generate http chunk install data.
	 *
	 * @param true if data should be generated, false otherwise.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetGenerateHttpChunkData(bool bGenerateHttpChunkData) = 0;
	/**
	 * Set where generated http chunk install data will be stored.
	 *
	 * @return the directory path to use.
	 */	
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetHttpChunkDataDirectory(const FString& InHttpChunkDataDirectory ) = 0;
	/**
	 * Set what name to tag the generated http chunk install data with.
	 *
	 * @param the name to use.
	 */	
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetHttpChunkDataReleaseName(const FString& InHttpChunkDataReleaseName ) = 0;

	/**
	 * Sets the device group to deploy to.
	 *
	 * @param DeviceGroup The device group, or NULL to reset this setting.
	 * @see GetDeployedDeviceGroup
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDeployedDeviceGroup( const ILauncherDeviceGroupPtr& DeviceGroup ) = 0;

	/**
	 * Sets the deployment mode.
	 *
	 * @param Mode The deployment mode to set.
	 * @see GetDeploymentMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDeploymentMode( ELauncherProfileDeploymentModes::Type Mode ) = 0;

	/**
	 * Creating a release version of the cooked content 
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsCreatingReleaseVersion() const = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCreateReleaseVersion(bool InCreateReleaseVersion) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetCreateReleaseVersionName() const = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCreateReleaseVersionName(const FString& InCreateReleaseVersionName) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetBasedOnReleaseVersionName() const = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetBasedOnReleaseVersionName(const FString& InBasedOnReleaseVersion) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetOriginalReleaseVersionName() const = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetOriginalReleaseVersionName(const FString& InOriginalReleaseVersion) = 0;

	/**
	* Provides a database of compressed iostore chunks to reuse during the
	* staging process. See IoStoreUtilities.cpp ReferenceContainerGlobalFileName.
	*/
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetReferenceContainerGlobalFileName() const = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetReferenceContainerGlobalFileName(const FString& InReferenceContainerGlobalFileName) = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetReferenceContainerCryptoKeysFileName() const = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetReferenceContainerCryptoKeysFileName(const FString& InReferenceContainerCryptoKeysFileName) = 0;

	/**
	 * Sets if we are going to generate a patch 
	 * 
	 * @param InShouldGeneratePatch enable generating patch
	 * @seealso IsGeneratingPatch
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetGeneratePatch( bool InShouldGeneratePatch ) = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetAddPatchLevel( bool InAddPatchLevel) = 0;
	UE_DEPRECATED(5.6, "ShouldStageBaseReleasePaks is no longer used")
	virtual void SetStageBaseReleasePaks(bool InStageBaseReleasePaks) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsCreatingDLC() const = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCreateDLC(bool InBuildDLC) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetDLCName() const = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDLCName(const FString& InDLCName) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsDLCIncludingEngineContent() const = 0;
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDLCIncludeEngineContent(bool InDLCIncludeEngineContent) = 0;


    /**
     * Sets the cook on the fly close mode
     *
     * @param Close the close mode to set.
     * @see GetForceClose
     */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
    virtual void SetForceClose( bool Close ) = 0;
    
	/**
	 * Sets whether to hide the file server's console window.
	 *
	 * @param Hide Whether to hide the window.
	 * @see GetHideFileServerWindow
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetHideFileServerWindow( bool Hide ) = 0;

	/**
	 * Sets legacy incremental cooking.
	 *
	 * @param Incremental Whether cooking should be incremental.
	 * @see IsCookingIncrementally
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetIncrementalCooking( bool Incremental ) = 0;

	/**
	 * Set the incremental cook mode.
	 * 
	 * @param Mode the incremental cooking mode to use
	 * @see GetIncrementalCookMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version with SetIncrementalCooking again instead")
	virtual void SetIncrementalCookMode( ELauncherProfileIncrementalCookMode::Type Mode ) = 0;

	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetIterateSharedCookedBuild( bool IterateSharedCookedBuild ) = 0;

	/**
	 * Sets Compression.
	 *
	 * @param Enable compression
	 * @see IsCompressed
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetCompressed( bool Enable ) = 0;


	/**
	 * Set encrypt ini files
	 *
	 * @param Enable encrypt ini files
	 * @see IsEncryptIniFiles
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetEncryptingIniFiles(bool Enabled) = 0;

	/**
	* Set this build is for distribution to the public
	*
	* @param enable for distribution
	* @see IsForDistribution
	*/
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetForDistribution(bool Enabled) = 0;
	

	/**
	 * Sets incremental deploying.
	 *
	 * @param Incremental Whether deploying should be incremental.
	 * @see IsDeployingIncrementally
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetIncrementalDeploying( bool Incremental ) = 0;

	/**
	 * Sets the launch mode.
	 *
	 * @param Mode The launch mode to set.
	 * @see GetLaunchMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetLaunchMode( ELauncherProfileLaunchModes::Type Mode ) = 0;

	/**
	 * Sets the packaging mode.
	 *
	 * @param Mode The packaging mode to set.
	 * @see GetPackagingMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetPackagingMode( ELauncherProfilePackagingModes::Type Mode ) = 0;

	/**
	 * Sets the packaging directory.
	 *
	 * @param Dir The packaging directory to set.
	 * @see GetPackageDirectory
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetPackageDirectory( const FString& Dir ) = 0;

	/**
	 * Sets whether to archive build
	 *
	 * @see GetArchiveMode
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetArchive( bool bArchive ) = 0;

	/**
	 * Sets the archive directory.
	 *
	 * @param Dir The archive directory to set.
	 * @see GetArchiveDirectory
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetArchiveDirectory( const FString& Dir ) = 0;

	/**
	 * Sets whether this profile specifies the a project.
	 *
	 * @param Specified Whether a project is specified.
	 */
	virtual void SetProjectSpecified(bool Specified) = 0;

	/** Notifies the profile that the fallback project path changed. */
	virtual void FallbackProjectUpdated() = 0;

	/**
	 * Sets the path to the Unreal project to use.
	 *
	 * @param Path The full path to the project.
	 * @see GetProjectPath
	 */
	virtual void SetProjectPath( const FString& Path ) = 0;

	/**
	 * Sets the additional command line parameters for the application to use at launch.
	 * These will be used by all launch roles
	 *
	 * @param	Params	The additional command line parameters to use
	 * @see GetAdditionalCommandLineParameters
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetAdditionalCommandLineParameters(const FString& Params) = 0;

	/**
	 * Sets the additional command line parameters that will be used for the given target type when the app launches.
	 *
	 * @param	Params	The additional command line parameters to use
	 * @param	BuildTargetType	The type of build target to modify
	 * @see GetAdditionalTargetCommandLineParameters
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetAdditionalTargetCommandLineParameters(const FString& Params, EBuildTargetType BuildTargetType ) = 0;


	/**
	 * Sets whether to use a streaming file server.
	 *
	 * @param Streaming Whether a streaming server should be used.
	 * @see GetStreamingFileServer
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetStreamingFileServer( bool Streaming ) = 0;

	/**
	 * Sets whether to include game prerequisites.
	 *
	 * @param Value Whether prerequisites should be used.
	 * @see IsIncludingPrerequisites
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetIncludePrerequisites(bool InValue) = 0;

    /**
     * Sets the cook on the fly server timeout
     *
     * @param InTime Amount of time to wait before timing out.
     * @see GetTimeout
     */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
    virtual void SetTimeout(uint32 InTime) = 0;
    
	/**
	 * Sets unversioned cooking.
	 *
	 * @param Unversioned Whether cooking is unversioned.
	 * @see IsCookingUnversioned
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetUnversionedCooking( bool Unversioned ) = 0;

	/**
	 * Accesses delegate used when the project changes.
	 *
	 * @return The delegate.
	 */
	virtual FOnProfileProjectChanged& OnProjectChanged() = 0;

	/**
	 * Access delegate used when build target options change.
	 * 
	 * @return The delegate.
	 */
	UE_DEPRECATED(5.8, "this was only used for the legacy project launcher")
	virtual FOnProfileBuildTargetOptionsChanged& OnBuildTargetOptionsChanged() = 0;

	/**
	 * Access delegate used when validation changes to gather custom warnings and errors
	 * 
	 * @return The delegate.
	 */
	virtual FOnProfileCustomValidation& OnCustomValidation() = 0;

	/**
	 * Access delegate used when validation changes to gather custom warnings and errors for custom UAT commands
	 * 
	 * @return The delegate.
	 */
	virtual FOnProfileCustomUATCommandValidation& OnCustomUATCommandValidation() = 0;

	/**
	 * Sets whether to use I/O store for optimized loading.
	 * @param bUseIoStore Whether to use I/O store
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetUseIoStore(bool bUseIoStore) = 0;

	/**
	 * Using I/O store or not.
	 *
	 * @return true if using I/O store
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsUsingIoStore() const = 0;

	/**
	 * Sets whether to use the Zen storage server. Note, this property is ignored if the project itself is configured to use Zen store (via UProjectPackagingSettings::bUseZenStore)
	 * @param bUseZenStore Whether to use the Zen storage server
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetUseZenStore(bool bUseZenStore) = 0;

	/**
	 * Using Zen storage server or not.
	 *
	 * @return true if using Zen storage server
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsUsingZenStore() const = 0;

	/**
	 * Get the Zen snapshot build number used for import.
	 *
	 * @return Zen snapshot build number
	 */
	virtual int32 GetZenSnapshot() = 0;

	/**
	 * Set whether to import a Zen snapshot before cooking.
	 * 
	 * @param Import Whether to import a Zen snapshot
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetImportingZenSnapshot( bool Import ) = 0;

	/**
	 * Checks whether a Zen snapshot will be imported before cooking.
	 * 
	 * @return true if importing a Zen snapshot, false otherwise
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsImportingZenSnapshot() const = 0;

	/**
	 * Set whether to use Zen pak streaming.
	 * 
	 * @param UseZenPakStreaming Whether use Zen pak streaming.
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetUseZenPakStreaming( bool UseZenPakStreaming ) = 0;

	/**
	 * Checks whether Zen pak streaming will be used.
	 * 
	 * @return true if using Zen pak streaming, false otherwise
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsUsingZenPakStreaming() const = 0;

	/**
	 * Sets whether to use Zen pak streaming from the given path.
	 * Assumes the path contains a suitable build
	 * 
	 * @param Path Path to the build to use for Zen pak streaming
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetZenPakStreamingPath( const FString& Path ) = 0;

	/**
	 * Get the current Zen pak streaming path, if it is being used.
	 * 
	 * @return current Zen pak streaming path, if any
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual FString GetZenPakStreamingPath() const = 0;

	/**
	 * Sets whether to make a binary config file during packaging
	 * @param bMakeBinaryConfig Whether to make a binary config file during staging
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetMakeBinaryConfig(bool bMakeBinaryConfig) = 0;
	/**
	 * Make binary config file during staging or not.
	 *
	 * @return true to make binary config file
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool MakeBinaryConfig() const = 0;

	/**
	 * Sets whether or not the flash image/software on the device should attempt to be updated before running
	 */
	virtual void SetShouldUpdateDeviceFlash(bool bInShouldUpdateFlash) = 0;

	/**
	 * Whether or not the flash image/software on the device should attempt to be updated before running
	 */
	virtual bool ShouldUpdateDeviceFlash() const = 0;

	/**
	 * Sets whether or not the Device is a Simulator
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetDeviceIsASimulator(bool bInIsDeviceASimualtor) = 0;

	/**
	 * Whether or not the Device is a Simulator
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual bool IsDeviceASimulator() const = 0;

	/** 
	 * Sets the client architecture(s) to build (x64, arm64 etc). Leave empty to use the default
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetClientArchitectures( const TArray<FString>& InArchitectures ) = 0;

	/**
	 * Returns the client architecture(s) that will be build. empty means the default will be used
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<FString>& GetClientArchitectures() const = 0;
	
	/** 
	 * Sets the server architecture(s) to build (x64, arm64 etc). Leave empty to use the default
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetServerArchitectures( const TArray<FString>& InArchitectures ) = 0;

	/**
	 * Returns the server architecture(s) that will be build. empty means the default will be used
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<FString>& GetServerArchitectures() const = 0;

	/** 
	 * Sets the editor architecture(s) to build (x64, arm64 etc). Leave empty to use the default
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void SetEditorArchitectures( const TArray<FString>& InArchitectures ) = 0;

	/**
	 * Returns the editor architecture(s) that will be build. empty means the default will be used
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual const TArray<FString>& GetEditorArchitectures() const = 0;





	
	/**
	 * Returns a list of all UAT commands in the profile
	 */
	virtual const TArray<ILauncherProfileUATCommandRef>& GetUATCommands() const = 0;

	/**
	 * Returns an existing UAT command by internal name
	 */
	virtual ILauncherProfileUATCommandPtr GetUATCommand( const TCHAR* InternalName ) const = 0;

	/**
	 * Returns an UAT command by internal name, creating it if it doesn't already exist
	 * Passing a null internal name will generate a new unique name
	 */
	virtual ILauncherProfileUATCommandRef FindOrAddUATCommand( const TCHAR* InternalName, const TCHAR* UserTypeName = nullptr ) = 0;

	/**
	 * Removes an UAT command by internal name
	 */
	virtual void RemoveUATCommand( const TCHAR* InternalName ) = 0;


	/**
	 * Callback for when a UAT command is added
	 */
	virtual FOnLauncherProfileUATCommandChanged& OnUATCommandAdded() = 0;

	/**
	 * Callback for when a UAT command is removed, prior to deletion
	 */
	virtual FOnLauncherProfileUATCommandChanged& OnUATCommandRemoved() = 0;







	/**
	 * Returns a list of all build cook run commands in the profile
	 */
	virtual const TArray<ILauncherProfileBuildCookRunRef> GetBuildCookRunCommands() const = 0;

	/**
	 * Return an existing UAT build cook run command by internal name
	 */
	virtual ILauncherProfileBuildCookRunPtr GetBuildCookRunCommand( const TCHAR* InternalName ) const = 0;

	/** 
	 * Returns a UAT build cook run command by internal name, creating it if it doesn't already exist
	 * Passing a null internal name will generate a new unique name
	 */
	virtual ILauncherProfileBuildCookRunRef FindOrAddBuildCookRunCommand( const TCHAR* InternalName, const TCHAR* UserTypeName = nullptr ) = 0;




	/**
	 * Returns a list of all automated tests in the profile
	 */
	virtual const TArray<ILauncherProfileAutomatedTestRef> GetAutomatedTests() const = 0;

	/**
	 * Returns an existing automated test by internal name
	 */
	virtual ILauncherProfileAutomatedTestPtr GetAutomatedTest( const TCHAR* InternalName ) const = 0;

	/**
	 * Returns an automated test by internal name, creating it if it doesn't already exist
	 * Passing a null internal name will generate a new unique name
	 */
	virtual ILauncherProfileAutomatedTestRef FindOrAddAutomatedTest( const TCHAR* InternalName, const TCHAR* UserTypeName = nullptr ) = 0;

	/**
	 * Removes an automated test by internal name
	 */
	virtual void RemoveAutomatedTest( const TCHAR* InternalName ) = 0;


	/**
	 * Sets the build to use for ALL automated tests, equivalent to the -build parameter on RunUnreal etc. See also SetIsUsingAutomatedTestBuild.
	 */
	virtual void SetAutomatedTestBuildPath( const FString& AutomatedTestBuildPath ) = 0;

	/**
	 * Returns the current automated test build path
	 */
	virtual const FString& GetAutomatedTestBuildPath() const = 0;

	/**
	 * Set whether to use the specified automated test build or the current launch build. Default is to use the current build.
	 */
	virtual void SetIsUsingAutomatedTestBuild( bool bWantAutomatedTestBuild ) = 0;

	/**
	 * Returns whether we are using the specified automated test build or not.
	 */
	virtual bool IsUsingAutomatedTestBuild() const = 0;




	/**
	 * Gets the names of the build targets
	 * @see AddBuildTarget, RemoveBuildTarget, ClearBuildTargets
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual TArray<FString> GetBuildTargets() const = 0;

	/**
	 * Add a build target to the list
	 * @see GetBuildTargets, RemoveBuildTarget, ClearBuildTargets
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void AddBuildTarget( const FString& InBuildTarget ) = 0;

	/**
	 * Remove a build target from the list
	 * @see GetBuildTargets, AddBuildTarget, ClearBuildTargets
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void RemoveBuildTarget( const FString& InBuildTarget ) = 0;

	/**
	 * Clears the list of build targets
	 * @see GetBuildTargets, AddBuildTarget, RemoveBuildTarget
	 */
	UE_DEPRECATED(5.8, "use GetFirstBuildCookRun version")
	virtual void ClearBuildTargets() = 0;



public:
	/**
	 * Helper function to get all of the build targets available for this profile, based on its current project & cook platforms
	 */
	UE_DEPRECATED(5.8, "this was only used for the legacy project launcher")
	virtual TArray<FString> GetExplicitBuildTargetNames() const = 0;

	/**
	 * Whether the profile will require an explicit -target=XXX parameter
	 */
	UE_DEPRECATED(5.8, "this was only used for the legacy project launcher")
	virtual bool RequiresExplicitBuildTargetName() const = 0;

	/**
	 * Key-Value pairs for extsibility. Not user-facing.
	 */
	virtual TMap<FString,FString>& GetCustomStringProperties() = 0;

	/**
	 * Key-Value pairs for extsibility. Not user-facing.
	 */
	virtual TMap<FString, bool>& GetCustomBoolProperties() = 0;

public:

	/** Whether this profile can be used by the legacy Project Launcher */
	virtual bool SupportsLegacyProjectLauncher() const = 0;

	/** Get access to the first build cook run command, if any */
	virtual ILauncherProfileBuildCookRunPtr GetFirstBuildCookRun() const = 0;

	/** Virtual destructor. */
	virtual ~ILauncherProfile( ) = default;
};

