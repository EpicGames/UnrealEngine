// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfileUATCommand.h"

class FJsonObject;

/**
 * Automated test parameters
 */
class ILauncherProfileAutomatedTest : public ILauncherProfileUATCommand
{
public:
	//@todo: comments

	virtual void SetTests( const TCHAR* Tests ) = 0; //-Test=xxx,yyy
	virtual const FString& GetTests() const = 0;

	virtual bool Load(const FJsonObject& Object) = 0;
	virtual void Save(TJsonWriter<>& Writer) = 0;

	UE_DEPRECATED(5.8, "please use the Json Save/Load API")
	virtual void Serialize( FArchive& Archive ) = 0;

	UE_DEPRECATED(5.8, "Use SetOrder")
	inline void SetPriority( int32 Priority ) { SetOrder(INT32_MAX-Priority); }

	virtual ~ILauncherProfileAutomatedTest() = default;
};

typedef TSharedPtr<ILauncherProfileAutomatedTest> ILauncherProfileAutomatedTestPtr;
typedef TSharedRef<ILauncherProfileAutomatedTest> ILauncherProfileAutomatedTestRef;
