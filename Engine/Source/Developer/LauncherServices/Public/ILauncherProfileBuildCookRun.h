// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "ILauncherDeviceGroup.h"
#include "ILauncherProfileLaunchRole.h"
#include "ILauncherProfileUATCommand.h"

class Error;
class FJsonObject;

namespace ELauncherProfileBuildModes
{
	/**
	 * Enumerates modes in which the launcher cooks builds.
	 */
	enum Type
	{
		/** Build if there is not already an existing pre-built target available. */
		Auto,

		/** Always build. */
		Build,

		/** Do not build. */
		DoNotBuild,
	};
}



namespace ELauncherProfileCookModes
{
	/**
	 * Enumerates modes in which the launcher cooks builds.
	 */
	enum Type
	{
		/** Do not cook the build (default). */
		DoNotCook,

		/** Pre-cook using user specified settings. */
		ByTheBook,

		/** Cook the build on the fly while the game is running. */
		OnTheFly,

		/** Cook by the book in the editor process space */
		ByTheBookInEditor,

		/** Cook on the fly in the editor process space */
		OnTheFlyInEditor,
	};
}


namespace ELauncherProfileCookedMaps
{
	/**
	 * Enumerates selections for maps to cook.
	 */
	enum Type
	{
		/** Cook all maps. */
		AllMaps,

		/** Don't cook any maps. Only startup packages will be cooked. */
		NoMaps,

		/** Cook user selected maps. */
		SelectedMaps

	};
}


namespace ELauncherProfileDeploymentModes
{
	/**
	 * Enumerates deployment modes.
	 */
	enum Type
	{
		/** Do not deploy the build to any device. */
		DoNotDeploy,

		/** Copy all required file to the device. */
		CopyToDevice,

		/** Let the device get required files from a file server. */
		FileServer,

		/** Copy a build from a repository to the device. */
		 CopyRepository,
	};
}


namespace ELauncherProfileLaunchModes
{
	/**
	 * Enumerates launch modes.
	 */
	enum Type
	{
		/** Do not launch. */
		DoNotLaunch,

		/** Launch with customized roles per device. */
		CustomRoles,

		/** Launch with the default role on all deployed devices. */
		DefaultRole	
	};
}


namespace ELauncherProfilePackagingModes
{
	/**
	 * Enumerates packaging modes.
	 */
	enum Type
	{
		/** Do not package. */
		DoNotPackage,

		/** Package and store the build locally. */
		Locally,

		/** Package and store the build in a shared repository. */
		SharedRepository
	};
}


namespace ELauncherProfileIncrementalCookMode
{
	/**
	 * Enumerates modes in which the launcher handles incremental cooking
	 */
	enum Type
	{
		/** Do not cook incrementally (default). */
		None,

		/** -iterativecooking : Only recook specifically modified assets without recooking dependencies. This is the legacy method and is must less robust. */
		ModifiedOnly,

		/** -cookincremental : Recook modified assets and any dependencies. This is the new method and is currently slower but far more robust. */
		ModifiedAndDependencies,
	};
}



namespace ELauncherProfileValidationErrors
{
	/**
	 * Enumerates profile validation messages.
	 */
	enum Type
	{
		/**
		 * Deployment by copying required files to a device requires
		 * cooking by the book and is incompatible with cook on the fly.
		 */
		CopyToDeviceRequiresCookByTheBook,

		/** Custom launch roles are not yet supported. */
		CustomRolesNotSupportedYet,

		/** A device group must be selected when deploying builds. */
		DeployedDeviceGroupRequired,
		
		/** The initial culture configured for launch is not part of the selected build. */
		InitialCultureNotAvailable,

		/** The initial map configured for launch is not part of the selected build. */
		InitialMapNotAvailable,

		/** The specified launch command line is not formatted correctly. */
		MalformedLaunchCommandLine,

		/** A build configuration is required when creating new builds. */
		NoBuildConfigurationSelected,

		/** When cooking a build, at least one culture must be included. */
		NoCookedCulturesSelected,

		/** One or more launch roles do not have a device assigned. */
		NoLaunchRoleDeviceAssigned,

		/** At least one platform is required when creating new builds. */
		NoPlatformSelected,

		/** A game is required when creating new builds. */
		NoProjectSelected,

		/** The deployment requires a package directory to be specified */
		NoPackageDirectorySpecified,
		
		/** The platform SDK is not installed but is required. */
		NoPlatformSDKInstalled,

		/** The profile has unversioned and incremental specified these are not compatible together */
		UnversionedAndIncremental,

		/** Deprecated, do not use */
		UnversionedAndIncrimental UE_DEPRECATED(5.6, "Update to use the correctly spelt 'ELauncherProfileValidationErrors::UnversionedAndIncremental' instead") = UnversionedAndIncremental,

		/** generating patch requires cook by the book mode*/
		GeneratingPatchesCanOnlyRunFromByTheBookCookMode,

		/** generating multilevel patch requires generating patch */
		GeneratingMultiLevelPatchesRequiresGeneratePatch,

		/** staging base release pak files requires a base release version to be specified */
		StagingBaseReleasePaksWithoutABaseReleaseVersion,

		/** Generating Chunks requires cook by the book mode */
		GeneratingChunksRequiresCookByTheBook,

		/** Generating Chunks requires UnrealPak */
		GeneratingChunksRequiresUnrealPak,

		/** Generating http chunk install data requires generating chunks or DLC*/
		GeneratingHttpChunkDataRequiresGeneratingChunks,

		/** Generating http chunk install data requires valid install directorys and release name */
		GeneratingHttpChunkDataRequiresValidDirectoryAndName,

		/** Shipping doesn't support commandline options can't use cook on the fly */
		ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly,

		/** Cook on the fly doesn't support server target platforms */
		CookOnTheFlyDoesntSupportServer,

		/** The archive step requires a directory to be specified */
		NoArchiveDirectorySpecified,

		/** Device is unauthorized or is locked */
		LaunchDeviceIsUnauthorized,
 
		/** Using I/O store container file(s) requires using UnrealPak */
		IoStoreRequiresPakFiles,

		/** Build Target and Cook Variant mismatch */
		BuildTargetCookVariantMismatch,

		/** Build Target is required */
		BuildTargetIsRequired,

		/** Fallback Build Target is required */
		FallbackBuildTargetIsRequired,

		/** Zen Pak Streaming requires deployment and launching */
		ZenPakStreamingRequiresDeployAndLaunch,

		/** Automated Tests currently require at least one active BuildCookRun */
		AutomatedTestRequiredBuildCookRun,

		Count
	};
}

LAUNCHERSERVICES_API FString LexToStringLocalized(ELauncherProfileValidationErrors::Type Value);




/** Type definition for shared pointers to instances of ILauncherProfileBuildCookRun. */
typedef TSharedPtr<class ILauncherProfileBuildCookRun> ILauncherProfileBuildCookRunPtr;

/** Type definition for shared references to instances of ILauncherProfileBuildCookRun. */
typedef TSharedRef<class ILauncherProfileBuildCookRun> ILauncherProfileBuildCookRunRef;

/** Delegate type for gathering custom profile validation */
DECLARE_MULTICAST_DELEGATE(FOnProfileCustomValidation);

/** Delegate type for gathering custom profile validation for a UAT command */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnProfileCustomUATCommandValidation, ILauncherProfileUATCommandRef);

/**
 * Interface for BuildCookRun UAT command.
 */
class ILauncherProfileBuildCookRun : public ILauncherProfileUATCommand
{
public:


	/**
	 * Checks whether devices of the specified platform can be deployed to.
	 *
	 * Whether a platform is deployable depends on the current profile settings.
	 * The right combination of build, cook and package settings must be present.
	 *
	 * @param PlatformName The name of the platform to deploy.
	 * @return true if the platform is deployable, false otherwise.
	 */
	virtual bool IsDeployablePlatform( const FString& PlatformName ) = 0;

	/**
	* Loads the profile from a JSON file
	*/
	virtual bool Load(const FJsonObject& Object) = 0;

	/**
	 * Saves the profile into a JSON file
	 */
	virtual void Save(TJsonWriter<>& Writer) = 0;

	/** Sets all profile settings to their defaults. */
	virtual void SetDefaults( ) = 0;

	/**
	 * Gets the identifier of the device group to deploy to.
	 *
	 * This method is used internally by the profile manager to read the device group identifier after
	 * loading this profile from a file. The profile manager will use this identifier to locate the
	 * actual device group to deploy to.
	 *
	 * @return The device group identifier, or an invalid GUID if no group was set or deployment is disabled.
	 */
	virtual const FGuid& GetDeployedDeviceGroupId() const = 0;

public:

	/**
	 * Gets the name of the build configuration.
	 *
	 * @return Build configuration name.
	 * @see SetBuildConfigurationName
	 */
	virtual EBuildConfiguration GetBuildConfiguration( ) const = 0;

	/**
	 * Checks whether the profile specifies a build target.
	 *
	 * @return true if the profile specifies a build target.
	 */
	virtual bool HasBuildTargetSpecified() const = 0;

	/**
	 * Gets the build configuration name of the cooker.
	 *
	 * @return Cook configuration name.
	 * @see SetCookConfigurationName
	 */
	virtual EBuildConfiguration GetCookConfiguration( ) const = 0;

	/**
	 * Gets the selected cook mode.
	 *
	 * @return Cook mode.
	 */
	virtual ELauncherProfileCookModes::Type GetCookMode( ) const = 0;

	/**
	 * Gets the cooker command line options.
	 *
	 * @return Cook options string.
	 */
	virtual const FString& GetCookOptions( ) const = 0;


	virtual const bool GetSkipCookingEditorContent() const = 0; 

	/**
	 * Skip editor content while cooking, 
	 * This will strip editor content from final builds
	 * 
	 * @param InSkipCookingEditorContent
	 */
	virtual void SetSkipCookingEditorContent(const bool InSkipCookingEditorContent) = 0;


	/**
	 * Gets the list of cooked culture.
	 *
	 * @return Collection of culture names.
	 * @see AddCookedCulture, ClearCookedCultures, RemoveCookedCulture
	 */
	virtual const TArray<FString>& GetCookedCultures( ) const = 0;

	/**
	 * Gets the list of cooked maps.
	 *
	 * @return Collection of map names.
	 *
	 * @see AddCookedMap, ClearCookedMaps, RemoveCookedMap
	 */
	virtual const TArray<FString>& GetCookedMaps( ) const = 0;

	/**
	 * Gets the names of the platforms to build for.
	 *
	 * @return Read-only collection of platform names.
	 *
	 * @see AddCookedPlatform, ClearCookedPlatforms, RemoveCookedPlatform
	 */
	virtual const TArray<FString>& GetCookedPlatforms( ) const = 0;

	/**
	 * Gets the default launch role.
	 *
	 * @return A reference to the default launch role.
	 */
	virtual const ILauncherProfileLaunchRoleRef& GetDefaultLaunchRole( ) const = 0;

	/**
	 * Gets the device group to deploy to.
	 *
	 * @param bRefreshDevices Whether to update the devices in the group before returning, in case some have been discovered since the last call
	 * @return The device group, or NULL if none was configured.
	 * @see SetDeployedDeviceGroup
	 */
	virtual ILauncherDeviceGroupPtr GetDeployedDeviceGroup( bool bRefreshDevices = true ) = 0;

	/**
	* Gets the default platforms to deploy if no specific devices were selected.
	*	
	*/
	virtual const FName GetDefaultDeployPlatform() const = 0;

	/**
	 * Gets the deployment mode.
	 *
	 * @return The deployment mode.
	 * @see SetDeploymentMode
	 */
	virtual ELauncherProfileDeploymentModes::Type GetDeploymentMode( ) const = 0;

    /**
     * Gets the close mode for the cook on the fly server
     *
     * @return The close mode.
     * @see SetForceClose
     */
    virtual bool GetForceClose() const = 0;

	/**
	 * Gets the launch mode.
	 *
	 * @return The launch mode.
	 * @see SetLaunchMode
	 */
	virtual ELauncherProfileLaunchModes::Type GetLaunchMode( ) const = 0;

	/**
	 * Gets the profile's collection of launch roles.
	 *
	 * @return A read-only collection of launch roles.
	 * @see CreateLaunchRole, RemoveLaunchRole
	 */
	virtual const TArray<ILauncherProfileLaunchRolePtr>& GetLaunchRoles( ) const = 0;

	/**
	 * Gets the launch roles assigned to the specified device.
	 *
	 * @param DeviceId The identifier of the device.
	 * @param OutRoles Will hold the assigned roles, if any.
	 */
	virtual const int32 GetLaunchRolesFor( const FString& DeviceId, TArray<ILauncherProfileLaunchRolePtr>& OutRoles ) = 0;

	/**
	 * Gets the packaging mode.
	 *
	 * @return The packaging mode.
	 * @see SetPackagingMode
	 */
	virtual ELauncherProfilePackagingModes::Type GetPackagingMode( ) const = 0;

	/**
	 * Gets the packaging directory.
	 *
	 * @return The packaging directory.
	 * @see SetPackageDirectory
	 */
	virtual FString GetPackageDirectory( ) const = 0;

	/**
	 * Whether to archive build
	 *
	 * @see SetArchive
	 */
	virtual bool IsArchiving( ) const = 0;

	/**
	 * Gets the archive directory.
	 *
	 * @return The archive directory.
	 * @see SetArchiveDirectory
	 */
	virtual FString GetArchiveDirectory( ) const = 0;

	/**
	 * Gets the additional command line parameters that will be used when the app launches.
	 * These will be used by all launch roles
	 *
	 * @return The additional command line parameters
	 * @see SetAdditionalCommandLineParameters
	 */
	virtual FString GetAdditionalCommandLineParameters() const = 0;

	/**
	 * Gets the additional command line parameters that will be used for the given target type when the app launches.
	 *
 	 * @param	BuildTargetType	The type of build target to query
	 * @return The additional command line parameters
	 * @see SetAdditionalTargetCommandLineParameters
	 */
	virtual FString GetAdditionalTargetCommandLineParameters( EBuildTargetType BuildTargetType ) const = 0;

    /**
     * Gets the timeout time for the cook on the fly server.
     *
     * @return The timeout time.
     * @see SetTimeout
     */
    virtual uint32 GetTimeout() const = 0;
	
	/**
	 * Are we going to generate a patch (Source content patch needs to be specified)
	 * @Seealso GetPatchSourceContentPath	 * @Seealso SetPatchSourceContentPath
	 */
	virtual bool IsGeneratingPatch() const = 0;
	
	/**
	 * Are we going to generate a new patch tier
	 */
	virtual bool ShouldAddPatchLevel() const = 0;

	/**
	 * Gets the current build mode.
	 *
	 * @return The current build mode.
	 * @see SetBuildMode
	 */
	virtual ELauncherProfileBuildModes::Type GetBuildMode() const = 0;

	/**
	 * Determines whether the current profile requires building
	 *
	 * @return The current build mode.
	 * @see SetBuildMode
	 */
	virtual bool ShouldBuild() = 0;

	/**
	 * Checks if fast iterate is enabled.
	 *
	 * @return true if fast iterate is enabled, false otherwise.
	 * @see SetFastIterate
	 */
	virtual bool IsFastIterate() const = 0;
	
	/**
	 * Checks whether legacy incremental cooking is enabled.
	 *
	 * @return true if cooking incrementally, false otherwise.
	 * @see SetIncrementalCooking
	 */
	virtual bool IsCookingIncrementally( ) const = 0;

	virtual bool IsIterateSharedCookedBuild() const =0;

	/**
	 * Checks if compression is enabled
	 *
	 * @return true if compression is enabled
	 * @see SetCompressed
	 */
	virtual bool IsCompressed( ) const = 0;

	/**
	 * Checks if encrypting ini files is enabled
	 * 
	 * @return true if encrypting ini files is enabled
	 */
	virtual bool IsEncryptingIniFiles() const = 0;

	/**
	 * Checks if encrypting ini files is enabled
	 *
	 * @return true if encrypting ini files is enabled
	 */
	virtual bool IsForDistribution() const = 0;

	/**
	 * Checks whether unversioned cooking is enabled.
	 *
	 * @return true if cooking unversioned, false otherwise.
	 * @see SetUnversionedCooking
	 */
	virtual bool IsCookingUnversioned( ) const = 0;

	/**
	 * Checks whether incremental deployment is enabled.
	 *
	 * @return true if deploying incrementally, false otherwise.
	 * @see SetIncrementalDeploying
	 */
	virtual bool IsDeployingIncrementally( ) const = 0;

	/**
	 * Checks whether the file server's console window should be hidden.
	 *
	 * @return true if the file server should be hidden, false otherwise.
	 * @see SetHideFileServer
	 */
	virtual bool IsFileServerHidden( ) const = 0;

	/**
	 * Checks whether the file server is a streaming file server.
	 *
	 * @return true if the file server is streaming, false otherwise.
	 * @see SetStreamingFileServer
	 */
	virtual bool IsFileServerStreaming( ) const = 0;

	/**
	 * Checks whether packaging with UnrealPak is enabled.
	 *
	 * @return true if UnrealPak is used, false otherwise.
	 * @see SetPackageWithUnrealPak
	 */
	virtual bool IsPackingWithUnrealPak( ) const = 0;

	/**
	 * Checks whether to include an installer for prerequisites of packaged games, such as redistributable operating system components, on platforms that support it.
	 *
	 * @return true if prerequisites are to be included, false otherwise.
	 * @see SetIncludePrerequisites
	 */
	virtual bool IsIncludingPrerequisites() const = 0;

	/**
	 * Return whether packaging will generate chunk data.
	 *
	 * @return true if Chunks will be generated, false otherwise.
	 */
	virtual bool IsGeneratingChunks() const = 0;
	
	/**
	 * Return whether packaging will use chunk data to generate http chunk install data.
	 *
	 * @return true if data will be generated, false otherwise.
	 */
	virtual bool IsGenerateHttpChunkData() const = 0;
	
	/**
	 * Where generated http chunk install data will be stored.
	 *
	 * @return true if data will be generated, false otherwise.
	 */
	virtual FString GetHttpChunkDataDirectory() const = 0;
	
	/**
	 * What name to tag the generated http chunk install data with.
	 *
	 * @return true if data will be generated, false otherwise.
	 */
	virtual FString GetHttpChunkDataReleaseName() const = 0;

public:

	/**
	 * Adds a culture to cook (only used if cooking by the book).
	 *
	 * @param CultureName The name of the culture to cook.
	 *
	 * @see ClearCookedCultures, GetCookedCultures, RemoveCookedCulture
	 */
	virtual void AddCookedCulture( const FString& CultureName ) = 0;

	/**
	 * Adds a map to cook (only used if cooking by the book).
	 *
	 * @param MapName The name of the map to cook.
	 *
	 * @see ClearCookedMaps, GetCookedMaps, RemoveCookedMap
	 */
	virtual void AddCookedMap( const FString& MapName ) = 0;

	/**
	 * Adds a platform to cook (only used if cooking by the book).
	 *
	 * @param PlatformName The name of the platform to add.
	 *
	 * @see ClearCookedPlatforms, GetCookedPlatforms, RemoveCookedPlatform
	 */
	virtual void AddCookedPlatform( const FString& PlatformName ) = 0;

	/**
	* Adds a platform to deploy (only used if a specific device is not specified).
	* Will deploy to the default device of the given platform, or the first device if none are
	* marked as 'default'.
	*
	* @param PlatformName The name of the platform to add.		
	*/
	virtual void SetDefaultDeployPlatform(const FName PlatformName) = 0;

	/**
	 * Removes all cooked cultures.
	 *
	 * @see AddCookedCulture, GetCookedCulture, RemoveCookedCulture
	 */
	virtual void ClearCookedCultures( ) = 0;

	/**
	 * Removes all cooked maps.
	 *
	 * @see AddCookedMap, GetCookedMap, RemoveCookedMap
	 */
	virtual void ClearCookedMaps( ) = 0;

	/**
	 * Removes all cooked platforms.
	 *
	 * @see AddCookedPlatform, GetCookedPlatforms, RemoveCookedPlatform
	 */
	virtual void ClearCookedPlatforms( ) = 0;

	/**
	 * Creates a new launch role and adds it to the profile.
	 *
	 * @return The created role.
	 * @see GetLaunchRoles, RemoveLaunchRole
	 */
	virtual ILauncherProfileLaunchRolePtr CreateLaunchRole( ) = 0;

	/**
	 * Removes a cooked culture.
	 *
	 * @param CultureName The name of the culture to remove.
	 * @see AddCookedCulture, ClearCookedCultures, GetCookedCultures
	 */
	virtual void RemoveCookedCulture( const FString& CultureName ) = 0;

	/**
	 * Removes a cooked map.
	 *
	 * @param MapName The name of the map to remove.
	 * @see AddCookedMap, ClearCookedMaps, GetCookedMaps
	 */
	virtual void RemoveCookedMap( const FString& MapName ) = 0;

	/**
	 * Removes a platform from the cook list.
	 *
	 * @param PlatformName The name of the platform to remove.
	 * @see AddBuildPlatform, ClearCookedPlatforms, GetBuildPlatforms
	 */
	virtual void RemoveCookedPlatform( const FString& PlatformName ) = 0;

	/**
	 * Removes the given launch role from the profile.
	 *
	 * @param Role The role to remove.
	 * @see CreateLaunchRole, GetLaunchRoles
	 */
	virtual void RemoveLaunchRole( const ILauncherProfileLaunchRoleRef& Role ) = 0;

	/**
	 * Sets the current build mode
	 *
	 * @param Mode Whether the game should be built.
	 * @see GetBuildMode
	 */
	virtual void SetBuildMode(ELauncherProfileBuildModes::Type Mode) = 0;

	/**
	 * Sets fast iterate.
	 *
	 * @param Enable fast iterate.
	 * @see IsFastIterate
	 */
	virtual void SetFastIterate(bool Enable) = 0;

	/**
	 * Sets the build configuration.
	 *
	 * @param ConfigurationName The build configuration name to set.
	 * @see GetBuildConfigurationName
	 */
	virtual void SetBuildConfiguration( EBuildConfiguration Configuration ) = 0;

	/**
	 * Sets whether this profile specfies a build target.
	 * 
	 * @param Specified Whether a build target is specified.
	 */
	virtual void SetBuildTargetSpecified(bool Specified) = 0;

	/**
	 * Sets the build configuration of the cooker.
	 *
	 * @param Configuration The cooker's build configuration to set.
	 * @see GetCookConfiguration
	 */
	virtual void SetCookConfiguration( EBuildConfiguration Configuration ) = 0;

	/**
	 * Sets the cook mode.
	 *
	 * @param Mode The cook mode.
	 * @see GetCookMode
	 */
	virtual void SetCookMode( ELauncherProfileCookModes::Type Mode ) = 0;

	/**
	 * Sets the cook options.
	 *
	 * @param Options The cook options.
	 * @see GetCookOptions
	 */
	virtual void SetCookOptions(const FString& Options) = 0;

	/**
	 * Sets whether to pack with UnrealPak.
	 *
	 * @param UseUnrealPak Whether UnrealPak should be used.
	 * @see IsPackingWithUnrealPak
	 */
	virtual void SetDeployWithUnrealPak( bool UseUnrealPak ) = 0;

	/**
	 * Set whether packaging will generate chunk data.
	 *
	 * @param true if Chunks should be generated, false otherwise.
	 */
	virtual void SetGenerateChunks(bool bGenerateChunks) = 0;
	/**
	 * Set whether packaging will use chunk data to generate http chunk install data.
	 *
	 * @param true if data should be generated, false otherwise.
	 */
	virtual void SetGenerateHttpChunkData(bool bGenerateHttpChunkData) = 0;
	/**
	 * Set where generated http chunk install data will be stored.
	 *
	 * @return the directory path to use.
	 */	
	virtual void SetHttpChunkDataDirectory(const FString& InHttpChunkDataDirectory ) = 0;
	/**
	 * Set what name to tag the generated http chunk install data with.
	 *
	 * @param the name to use.
	 */	
	virtual void SetHttpChunkDataReleaseName(const FString& InHttpChunkDataReleaseName ) = 0;

	/**
	 * Sets the device group to deploy to.
	 *
	 * @param DeviceGroup The device group, or NULL to reset this setting.
	 * @see GetDeployedDeviceGroup
	 */
	virtual void SetDeployedDeviceGroup( const ILauncherDeviceGroupPtr& DeviceGroup ) = 0;

	/**
	 * Sets the deployment mode.
	 *
	 * @param Mode The deployment mode to set.
	 * @see GetDeploymentMode
	 */
	virtual void SetDeploymentMode( ELauncherProfileDeploymentModes::Type Mode ) = 0;

	/**
	 * Creating a release version of the cooked content 
	 */
	virtual bool IsCreatingReleaseVersion() const = 0;

	virtual void SetCreateReleaseVersion(bool InCreateReleaseVersion) = 0;

	virtual FString GetCreateReleaseVersionName() const = 0;

	virtual void SetCreateReleaseVersionName(const FString& InCreateReleaseVersionName) = 0;

	virtual FString GetBasedOnReleaseVersionName() const = 0;

	virtual void SetBasedOnReleaseVersionName(const FString& InBasedOnReleaseVersion) = 0;

	virtual FString GetOriginalReleaseVersionName() const = 0;

	virtual void SetOriginalReleaseVersionName(const FString& InOriginalReleaseVersion) = 0;

	/**
	* Provides a database of compressed iostore chunks to reuse during the
	* staging process. See IoStoreUtilities.cpp ReferenceContainerGlobalFileName.
	*/
	virtual FString GetReferenceContainerGlobalFileName() const = 0;
	virtual void SetReferenceContainerGlobalFileName(const FString& InReferenceContainerGlobalFileName) = 0;
	virtual FString GetReferenceContainerCryptoKeysFileName() const = 0;
	virtual void SetReferenceContainerCryptoKeysFileName(const FString& InReferenceContainerCryptoKeysFileName) = 0;

	/**
	 * Sets if we are going to generate a patch 
	 * 
	 * @param InShouldGeneratePatch enable generating patch
	 * @seealso IsGeneratingPatch
	 */
	virtual void SetGeneratePatch( bool InShouldGeneratePatch ) = 0;
	virtual void SetAddPatchLevel( bool InAddPatchLevel) = 0;

	virtual bool IsCreatingDLC() const = 0;
	virtual void SetCreateDLC(bool InBuildDLC) = 0;

	virtual FString GetDLCName() const = 0;
	virtual void SetDLCName(const FString& InDLCName) = 0;

	virtual bool IsDLCIncludingEngineContent() const = 0;
	virtual void SetDLCIncludeEngineContent(bool InDLCIncludeEngineContent) = 0;


    /**
     * Sets the cook on the fly close mode
     *
     * @param Close the close mode to set.
     * @see GetForceClose
     */
    virtual void SetForceClose( bool Close ) = 0;
    
	/**
	 * Sets whether to hide the file server's console window.
	 *
	 * @param Hide Whether to hide the window.
	 * @see GetHideFileServerWindow
	 */
	virtual void SetHideFileServerWindow( bool Hide ) = 0;

	/**
	 * Sets legacy incremental cooking.
	 *
	 * @param Incremental Whether cooking should be incremental.
	 * @see IsCookingIncrementally
	 */
	virtual void SetIncrementalCooking( bool Incremental ) = 0;


	virtual void SetIterateSharedCookedBuild( bool IterateSharedCookedBuild ) = 0;

	/**
	 * Sets Compression.
	 *
	 * @param Enable compression
	 * @see IsCompressed
	 */
	virtual void SetCompressed( bool Enable ) = 0;


	/**
	 * Set encrypt ini files
	 *
	 * @param Enable encrypt ini files
	 * @see IsEncryptIniFiles
	 */
	virtual void SetEncryptingIniFiles(bool Enabled) = 0;

	/**
	* Set this build is for distribution to the public
	*
	* @param enable for distribution
	* @see IsForDistribution
	*/
	virtual void SetForDistribution(bool Enabled) = 0;
	

	/**
	 * Sets incremental deploying.
	 *
	 * @param Incremental Whether deploying should be incremental.
	 * @see IsDeployingIncrementally
	 */
	virtual void SetIncrementalDeploying( bool Incremental ) = 0;

	/**
	 * Sets the launch mode.
	 *
	 * @param Mode The launch mode to set.
	 * @see GetLaunchMode
	 */
	virtual void SetLaunchMode( ELauncherProfileLaunchModes::Type Mode ) = 0;

	/**
	 * Sets the packaging mode.
	 *
	 * @param Mode The packaging mode to set.
	 * @see GetPackagingMode
	 */
	virtual void SetPackagingMode( ELauncherProfilePackagingModes::Type Mode ) = 0;

	/**
	 * Sets the packaging directory.
	 *
	 * @param Dir The packaging directory to set.
	 * @see GetPackageDirectory
	 */
	virtual void SetPackageDirectory( const FString& Dir ) = 0;

	/**
	 * Sets whether to archive build
	 *
	 * @see GetArchiveMode
	 */
	virtual void SetArchive( bool bArchive ) = 0;

	/**
	 * Sets the archive directory.
	 *
	 * @param Dir The archive directory to set.
	 * @see GetArchiveDirectory
	 */
	virtual void SetArchiveDirectory( const FString& Dir ) = 0;

	/**
	 * Sets the additional command line parameters for the application to use at launch.
	 * These will be used by all launch roles
	 *
	 * @param	Params	The additional command line parameters to use
	 * @see GetAdditionalCommandLineParameters
	 */
	virtual void SetAdditionalCommandLineParameters(const FString& Params) = 0;

	/**
	 * Sets the additional command line parameters that will be used for the given target type when the app launches.
	 *
	 * @param	Params	The additional command line parameters to use
	 * @param	BuildTargetType	The type of build target to modify
	 * @see GetAdditionalTargetCommandLineParameters
	 */
	virtual void SetAdditionalTargetCommandLineParameters(const FString& Params, EBuildTargetType BuildTargetType ) = 0;


	/**
	 * Sets whether to use a streaming file server.
	 *
	 * @param Streaming Whether a streaming server should be used.
	 * @see GetStreamingFileServer
	 */
	virtual void SetStreamingFileServer( bool Streaming ) = 0;

	/**
	 * Sets whether to include game prerequisites.
	 *
	 * @param Value Whether prerequisites should be used.
	 * @see IsIncludingPrerequisites
	 */
	virtual void SetIncludePrerequisites(bool InValue) = 0;

    /**
     * Sets the cook on the fly server timeout
     *
     * @param InTime Amount of time to wait before timing out.
     * @see GetTimeout
     */
    virtual void SetTimeout(uint32 InTime) = 0;
    
	/**
	 * Sets unversioned cooking.
	 *
	 * @param Unversioned Whether cooking is unversioned.
	 * @see IsCookingUnversioned
	 */
	virtual void SetUnversionedCooking( bool Unversioned ) = 0;

	/**
	 * Sets whether to use I/O store for optimized loading.
	 * @param bUseIoStore Whether to use I/O store
	 */
	virtual void SetUseIoStore(bool bUseIoStore) = 0;

	/**
	 * Using I/O store or not.
	 *
	 * @return true if using I/O store
	 */
	virtual bool IsUsingIoStore() const = 0;

	/**
	 * Sets whether to use the Zen storage server. Note, this property is ignored if the project itself is configured to use Zen store (via UProjectPackagingSettings::bUseZenStore)
	 * @param bUseZenStore Whether to use the Zen storage server
	 */
	virtual void SetUseZenStore(bool bUseZenStore) = 0;

	/**
	 * Using Zen storage server or not.
	 *
	 * @return true if using Zen storage server
	 */
	virtual bool IsUsingZenStore() const = 0;

	/**
	 * Set whether to import a Zen snapshot before cooking.
	 * 
	 * @param Import Whether to import a Zen snapshot
	 */
	virtual void SetImportingZenSnapshot( bool Import ) = 0;

	/**
	 * Checks whether a Zen snapshot will be imported before cooking.
	 * 
	 * @return true if importing a Zen snapshot, false otherwise
	 */
	virtual bool IsImportingZenSnapshot() const = 0;

	/**
	 * Checks whether Zen streaming will be used.
	 * 
	 * @return true if using Zen streaming, false otherwise
	 */
	virtual bool IsUsingZenStreaming() const = 0;
	
	/**
	 * Set whether to use Zen streaming.
	 *
	 * @param UseZenStreaming Whether use Zen stremaing.
	 */
	virtual void SetUseZenStreaming( bool UseZenStreaming ) = 0;

	/**
	 * Set whether to use Zen pak streaming.
	 * 
	 * @param UseZenPakStreaming Whether use Zen pak streaming.
	 */
	virtual void SetUseZenPakStreaming( bool UseZenPakStreaming ) = 0;

	/**
	 * Checks whether Zen pak streaming will be used.
	 * 
	 * @return true if using Zen pak streaming, false otherwise
	 */
	virtual bool IsUsingZenPakStreaming() const = 0;

	/**
	 * Sets whether to use Zen pak streaming from the given path.
	 * Assumes the path contains a suitable build
	 * 
	 * @param Path Path to the build to use for Zen pak streaming
	 */
	virtual void SetZenPakStreamingPath( const FString& Path ) = 0;

	/**
	 * Get the current Zen pak streaming path, if it is being used.
	 * 
	 * @return current Zen pak streaming path, if any
	 */
	virtual FString GetZenPakStreamingPath() const = 0;

	/**
	 * Set whether to use a pre-staged build
	 * 
	 * @param UsePreStagedBuild Whether use a pre-staged build
	 */
	virtual void SetUsePreStagedBuild( bool UsePreStagedBuild ) = 0;

	/**
	 * Checks whether a pre-staged build willbe used
	 * 
	 * @return true if using a pre-staged build, false otherwise
	 */
	virtual bool IsUsingPreStagedBuild() const = 0;

	/**
	 * Sets whether to make a binary config file during packaging
	 * @param bMakeBinaryConfig Whether to make a binary config file during staging
	 */
	virtual void SetMakeBinaryConfig(bool bMakeBinaryConfig) = 0;
	/**
	 * Make binary config file during staging or not.
	 *
	 * @return true to make binary config file
	 */
	virtual bool MakeBinaryConfig() const = 0;

	/**
	 * Sets whether or not the Device is a Simulator
	 */
	virtual void SetDeviceIsASimulator(bool bInIsDeviceASimualtor) = 0;

	/**
	 * Whether or not the Device is a Simulator
	 */
	virtual bool IsDeviceASimulator() const = 0;

	/** 
	 * Sets the client architecture(s) to build (x64, arm64 etc). Leave empty to use the default
	 */
	virtual void SetClientArchitectures( const TArray<FString>& InArchitectures ) = 0;

	/**
	 * Returns the client architecture(s) that will be build. empty means the default will be used
	 */
	virtual const TArray<FString>& GetClientArchitectures() const = 0;
	
	/** 
	 * Sets the server architecture(s) to build (x64, arm64 etc). Leave empty to use the default
	 */
	virtual void SetServerArchitectures( const TArray<FString>& InArchitectures ) = 0;

	/**
	 * Returns the server architecture(s) that will be build. empty means the default will be used
	 */
	virtual const TArray<FString>& GetServerArchitectures() const = 0;

	/** 
	 * Sets the editor architecture(s) to build (x64, arm64 etc). Leave empty to use the default
	 */
	virtual void SetEditorArchitectures( const TArray<FString>& InArchitectures ) = 0;

	/**
	 * Returns the editor architecture(s) that will be build. empty means the default will be used
	 */
	virtual const TArray<FString>& GetEditorArchitectures() const = 0;





	/**
	 * Gets the names of the build targets
	 * @see AddBuildTarget, RemoveBuildTarget, ClearBuildTargets
	 */
	virtual TArray<FString> GetBuildTargets() const = 0;

	/**
	 * Add a build target to the list
	 * @see GetBuildTargets, RemoveBuildTarget, ClearBuildTargets
	 */
	virtual void AddBuildTarget( const FString& InBuildTarget ) = 0;

	/**
	 * Remove a build target from the list
	 * @see GetBuildTargets, AddBuildTarget, ClearBuildTargets
	 */
	virtual void RemoveBuildTarget( const FString& InBuildTarget ) = 0;

	/**
	 * Clears the list of build targets
	 * @see GetBuildTargets, AddBuildTarget, RemoveBuildTarget
	 */
	virtual void ClearBuildTargets() = 0;




public:

	/** Virtual destructor. */
	virtual ~ILauncherProfileBuildCookRun( ) = default;
};

