// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/JsonWriter.h"

class FJsonObject;
class ILauncherProfileLaunchRole;

namespace ELauncherProfileRoleInstanceTypes
{
	/**
	 * Enumerates launch role instance types.
	 */
	enum Type
	{
		/** The instance is a dedicated server. */
		DedicatedServer,

		/** The instance is a listen server. */
		ListenServer,

		/** The instance is a game client. */
		StandaloneClient,

		/** The instance is an Unreal Editor. */
		UnrealEditor
	};

	/**
	 * Returns the string representation of the specified ELauncherProfileRoleInstanceTypes value.
	 *
	 * @param InstanceType - The value to get the string for.
	 *
	 * @return A string value.
	 */
	inline FString ToString( ELauncherProfileRoleInstanceTypes::Type InstanceType )
	{
		switch (InstanceType)
		{
			case DedicatedServer:
				return FString("Dedicated Server");

			case ListenServer:
				return FString("Listen Server");

			case StandaloneClient:
				return FString("Standalone Client");

			case UnrealEditor:
				return FString("Unreal Editor");

			default:
				return FString();
		}
	}
}


/** Type definition for shared pointers to instances of ILauncherProfileLaunchRole. */
typedef TSharedPtr<class ILauncherProfileLaunchRole> ILauncherProfileLaunchRolePtr;

/** Type definition for shared references to instances of ILauncherProfileLaunchRole. */
typedef TSharedRef<class ILauncherProfileLaunchRole> ILauncherProfileLaunchRoleRef;


/**
 * Interface for launch roles.
 */
class ILauncherProfileLaunchRole
{
public:

	/**
	 * Gets the identifier of the device that is assigned to this role.
	 *
	 * @return The device identifier, or an empty string if the role is unassigned.
	 */
	virtual const FString& GetAssignedDevice( ) const = 0;

	/**
	 * Gets optional command line parameters to launch with.
	 *
	 * @return The command line string.
	 * @see SetCommandLine
	 */
	virtual const FString& GetUATCommandLine( ) const = 0;

	/**
	 * Gets the initial culture to launch with.
	 *
	 * @return Culture name.
	 * @see SetInitialCulture
	 */
	virtual const FString& GetInitialCulture( ) const = 0;

	/**
	 * Gets the initial map to launch with.
	 *
	 * @return Map name.
	 * @see SetInitialMap
	 */
	virtual const FString& GetInitialMap( ) const = 0;

	/**
	 * Gets the instance type (i.e. client, server, etc).
	 *
	 * @return Instance type.
	 */
	virtual ELauncherProfileRoleInstanceTypes::Type GetInstanceType( ) const = 0;

	/**
	 * Gets the name of this role.
	 *
	 * @return The role name.
	 */
	virtual const FString& GetName( ) const = 0;

	/**
	 * Checks whether vertical sync is enabled.
	 *
	 * @return true if VSync is enabled, false otherwise.
	 * @see SetVsyncEnabled
	 */
	virtual bool IsVsyncEnabled( ) const = 0;

	virtual void Load(const FJsonObject& Object) = 0;

	virtual void Save(TJsonWriter<>& Writer, const TCHAR* Name = TEXT("")) = 0;

	/**
	 * Serializes the role from or into the specified archive.
	 *
	 * @param Archive The archive to serialize from or into.
	 */
	UE_DEPRECATED(5.8, "please use the Json Save/Load API")
	virtual void Serialize( FArchive& Archive ) = 0;

	/**
	 * Sets optional command line parameters to launch with.
	 *
	 * @param NewCommandLine Command line string.
	 * @see GetUATCommandLine
	 */
	virtual void SetCommandLine( const FString& NewCommandLine ) = 0;

	/**
	 * Sets the initial culture to launch with.
	 *
	 * @param CultureName The culture name.
	 * @see GetInitialCulture
	 */
	virtual void SetInitialCulture( const FString& CultureName ) = 0;

	/**
	 * Sets the initial map to launch with.
	 *
	 * @param MapName The name of the map.
	 * @see GetInitialMap
	 */
	virtual void SetInitialMap( const FString& MapName ) = 0;

	/**
	 * Sets the role instance type (i.e. client, server, etc).
	 *
	 * @param InInstanceType The instance type to set.
	 */
	virtual void SetInstanceType( ELauncherProfileRoleInstanceTypes::Type InInstanceType ) = 0;

	/**
	 * Sets the name of this role.
	 *
	 * @param NewName The name to set.
	 * @see GetName
	 */
	virtual void SetName( const FString& NewName ) = 0;

	/**
	 * Sets whether vertical sync should be enabled.
	 *
	 * @param Enabled Whether VSync is enabled.
	 * @see IsVsyncEnabled
	 */
	virtual void SetVsyncEnabled( bool Enabled ) = 0;

public:

	/** Virtual destructor. */
	virtual ~ILauncherProfileLaunchRole( ) = default;
};
