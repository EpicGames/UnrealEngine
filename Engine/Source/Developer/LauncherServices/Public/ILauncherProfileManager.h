// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherProfile.h"

class ILauncherProfileManager;
class ILauncherProfileWizard;
struct FCommandDesc;

/** Type definition for shared pointers to instances of ILauncherProfileManager. */
typedef TSharedPtr<class ILauncherProfileManager> ILauncherProfileManagerPtr;

/** Type definition for shared references to instances of ILauncherProfileManager. */
typedef TSharedRef<class ILauncherProfileManager> ILauncherProfileManagerRef;

/** Parameters for customizing a UAT command via a callback */
struct FUATCommandPostProcessParameters
{
	bool bShouldBeSkipped;

	const ILauncherProfileRef Profile;
	const ILauncherProfileUATCommandRef UATCommand;
	FString& UATCommandLine;
	
	TArray<FCommandDesc>& Commands;
	FString& CommandStart;
};


/**
 * Declares a delegate to be invoked when a device group was added to a profile manager.
 *
 * The first parameter is the profile that was added.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileManagerDeviceGroupAdded, const ILauncherDeviceGroupRef&);

/**
 * Declares a delegate to be invoked when a device group was removed from a profile manager.
 *
 * The first parameter is the profile that was removed.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileManagerDeviceGroupRemoved, const ILauncherDeviceGroupRef&);

/**
 * Declares a delegate to be invoked when a launcher profile was added to a profile manager.
 *
 * The first parameter is the profile that was added.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileManagerProfileAdded, const ILauncherProfileRef&);

/**
 * Declares a delegate to be invoked when a launcher profile was removed from a profile manager.
 *
 * The first parameter is the profile that was removed.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileManagerProfileRemoved, const ILauncherProfileRef&);


/**
 * Declares a delegate to be invoked when the project is changed.
 *
 */
DECLARE_MULTICAST_DELEGATE(FOnLauncherProfileManagerProjectChanged);

/**
 * Declares a delegate to be invoked to post-process a command line before launching
 *
 * The first parameter is the profile that is being launched, the second parameter is the build target type (or 'unknown' for global arguments), and the third parameter is the command line to modify.
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLauncherProfilePostProcessCommandLine, const ILauncherProfileRef&, EBuildTargetType, FString& );

/**
 * Declares a delegate to be invoked to post-process a command line before launching and automated test
 *
 * The first parameter is the automated test, the second is the profile that is being launched, the third parameter is the command line to modify.
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLauncherProfilePostProcessAutomatedTestCommandLine, const ILauncherProfileAutomatedTestRef&, const ILauncherProfileRef&, FString& );

/**
 * Declares a delegate to be invoked to post-process a command line before launching a UAT command
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfilePostProcessUATCommandLine, FUATCommandPostProcessParameters& );


/** Type definition for shared references to instances of ILauncherProfile. */
typedef TSharedPtr<class ILauncherProfileWizard> ILauncherProfileWizardPtr;

/** Type definition for shared references to instances of ILauncherProfile. */
typedef TSharedRef<class ILauncherProfileWizard> ILauncherProfileWizardRef;

/**
* Interface for a factory to create pre-defined launcher profiles.
*/
class ILauncherProfileWizard
{
public:
	/**
	* Wizard name that will be used for menu
	*/
	virtual FText GetName() const = 0;
	
	/**
	* Wizard description text that will be used for menu tooltip
	*/
	virtual FText GetDescription() const = 0;

	/**
	* Handle request to create launcher profile using this wizard
	*
	* @param ProfileManager The profile manager initiated this request.
	*/
	virtual void HandleCreateLauncherProfile(const ILauncherProfileManagerRef& ProfileManager) = 0;

public:
	/** Virtual destructor. */
	virtual ~ILauncherProfileWizard( ) = default;
};

/**
 * Interface for launcher profile managers.
 */
class ILauncherProfileManager
{
public:

	/**
	 * Adds the given device group.
	 *
	 * @param DeviceGroup The group to add.
	 */
	virtual void AddDeviceGroup( const ILauncherDeviceGroupRef& DeviceGroup ) = 0;

	/**
	 * Create a new device group and maintains a reference for its future usage.
	 *
	 * @return The device group created.
	 */
	virtual ILauncherDeviceGroupRef AddNewDeviceGroup( ) = 0;

	/**
	 * Creates a new device group but does not add it to the internal tracking.
	 *
	 * @return The new device group created.
	 */
	virtual ILauncherDeviceGroupRef CreateUnmanagedDeviceGroup() = 0;

	/**
	 * Gets the collection of device groups.
	 *
	 * @return A read-only collection of device groups.
	 */
	virtual const TArray<ILauncherDeviceGroupPtr>& GetAllDeviceGroups( ) const = 0;

	/**
	 * Gets the device group with the specified identifier.
	 *
	 * @param GroupId The unique identifier of the group to get.
	 * @return A shared pointer to the group, or nullptr if the group was not found.
	 */
	virtual ILauncherDeviceGroupPtr GetDeviceGroup( const FGuid& GroupId ) const = 0;

	/** 
	 * Deletes the specified device group.
	 *
	 * @param DeviceGroup The group to remove.
	 */
	virtual void RemoveDeviceGroup( const ILauncherDeviceGroupRef& DeviceGroup ) = 0;

	/** Saves all the device groups to a config file */
	virtual void SaveDeviceGroups( bool bTrimUnused = false ) = 0;

public:

	/**
	 * Finds or Adds then returns a simple profile for the specified Device
	 *
	 * @param DeviceName The name of the device we want the simple profile for.
	 * @return The simple profile for the specified device.
	 */
	virtual ILauncherSimpleProfilePtr FindOrAddSimpleProfile(const FString& DeviceName) = 0;

	/**
	 * Gets the simple profile for the specified device.
	 *
	 * @param DeviceName The name of the device we want the simple profile for.
	 * @return The simple profile for the specified device, or nullptr if the simple profile doesn't exist.
	 */
	virtual ILauncherSimpleProfilePtr FindSimpleProfile(const FString& DeviceName) = 0;

	/**
	 * Deletes the given simple profile.
	 *
	 * @param Profile The simple profile to delete.
	 */
	virtual void RemoveSimpleProfile(const ILauncherSimpleProfileRef& SimpleProfile) = 0;

public:

	/**
	 * Creates a new profile.
	 *
	 * @return The new profile created.
	 */
	virtual ILauncherProfileRef AddNewProfile( ) = 0;

	/**
	 * Creates a new profile but does not add it to the internal tracking.
	 *
	 * @param ProfileName The name of the profile to create.
	 * @return The new profile created.
	 */
	virtual ILauncherProfileRef CreateUnsavedProfile(FString ProfileName) = 0;

	/**
	 * Adds the given profile to the list of managed profiles.
	 *
	 * If a profile with the same identifier already exists in the profile
	 * collection, it will be deleted before the given profile is added.
	 *
	 * @param Profile The profile to add.
	 */
	virtual void AddProfile( const ILauncherProfileRef& Profile ) = 0;

	/**
	 * Gets the profile with the specified name.
	 *
	 * @param ProfileName The name of the profile to get.
	 * @return The profile, or nullptr if the profile doesn't exist.
	 * @see GetProfile
	 */
	virtual ILauncherProfilePtr FindProfile( const FString& ProfileName ) = 0;

	/**
	 * Gets the collection of profiles.
	 *
	 * @return A read-only collection of profiles.
	 *
	 * @see GetSelectedProfile
	 */
	virtual const TArray<ILauncherProfilePtr>& GetAllProfiles( ) const = 0;

	/**
	 * Gets the profile with the specified identifier.
	 *
	 * @param ProfileId The identifier of the profile to get.
	 * @return The profile, or nullptr if the profile doesn't exist.
	 * @see FindProfile
	 */
	virtual ILauncherProfilePtr GetProfile( const FGuid& ProfileId ) const = 0;

	/**
	 * Attempts to load a profile from the specified archive.
	 *
	 * The loaded profile is NOT automatically added to the profile manager.
	 * Use AddProfile() to add it to the collection.
	 *
	 * @param Archive The archive to load from.
	 * @return The loaded profile, or nullptr if loading failed.
	 * @see AddProfile, SaveProfile
	 */
	UE_DEPRECATED(5.8, "Please use LoadJSONProfile")
	virtual ILauncherProfilePtr LoadProfile( FArchive& Archive ) = 0;

	/**
	* Attempts to load a profile from the specified file.
	*
	* The loaded profile is NOT automatically added to the profile manager.
	* Use AddProfile() to add it to the collection.
	*
	* @param ProfileFile The file to load from.
	* @return The loaded profile, or nullptr if loading failed.
	* @see AddProfile, SaveProfile
	*/
	virtual ILauncherProfilePtr LoadJSONProfile(FString ProfileFile) = 0;
	/**
	 * Deletes the given profile.
	 *
	 * @param Profile The profile to delete.
	 */
	virtual void RemoveProfile( const ILauncherProfileRef& Profile ) = 0;

	/**
	 * Saves the given profile to the specified archive.
	 *
	 * @param Profile The profile to save.
	 * @param Archive The archive to save to.
	 * @see LoadProfile
	 */
	UE_DEPRECATED(5.8, "Please use SaveJSONProfile")
	virtual bool SaveProfile(const ILauncherProfileRef& Profile) = 0;

	/**
	* Saves the given profile to the specified file.
	*
	* @param Profile The profile to save.
	* @see LoadJSONProfile
	*/
	virtual bool SaveJSONProfile(const ILauncherProfileRef& Profile) = 0;

	/**
	* Modifies profile name. Saves file to new location and deletes old one to avoid duplicating profiles.
	*
	* @param Profile The profile to change	
	*/
	virtual void ChangeProfileName(const ILauncherProfileRef& Profile, FString Name) = 0;

	/**
	* Register wizard that can be used to create pre-defined launcher profiles
	*
	* @param ProfileWizard The wizard to register
	*/
	virtual void RegisterProfileWizard(const ILauncherProfileWizardPtr& ProfileWizard) = 0;

	/**
	* Unregister launcher profile wizard
	*
	* @param ProfileWizard The wizard to unregister
	*/
	virtual void UnregisterProfileWizard(const ILauncherProfileWizardPtr& ProfileWizard) = 0;

	/**
	* Return list of all registered profile wizards
	*/
	virtual const TArray<ILauncherProfileWizardPtr>& GetProfileWizards() const = 0;

	/**
	 * Returns best-guess BuildCookRun parameters for the given profile in the format expected by FProjectBuidSettings::BuildCookRunParams
	 *  Used by the Project custom Build's "import from Project Launcher" menu
	 */
	virtual FString MakeBuildCookRunParamsForProjectCustomBuild(const ILauncherProfileRef& InProfile, const TArray<FString>& InPlatforms) const = 0;

public:

	/**
	 * Loads all device groups and launcher profiles from disk.
	 *
	 * When this function is called, it will discard any in-memory changes to device groups
	 * and launcher profiles that are not yet persisted to disk. Settings are also loaded
	 * automatically when a profile manager is first created.
	 *
	 * @see SaveSettings
	 */
	virtual void LoadSettings( ) = 0;

	/**
	 * Persists all device groups, launcher profiles and other settings to disk.\
	 *
	 * @see LoadSettings
	 */
	virtual void SaveSettings( ) = 0;

public:

	/**
	 * Gets the name of the Unreal project to use.
	 */
	virtual FString GetProjectName() const = 0;

	/**
	 * Gets the base project path for the project
	 */
	virtual FString GetProjectBasePath() const = 0;

	/**
	 * Gets the full path to the Unreal project to use.
	 *
	 * @return The path.
	 * @see SetProjectPath
	 */
	virtual FString GetProjectPath() const = 0;

	/**
	 * Sets the path to the Unreal project to use.
	 *
	 * @param Path The full path to the project.
	 * @see GetProjectPath
	 */
	virtual void SetProjectPath(const FString& InProjectPath) = 0;

public:

	/**
	 * Gets the default build target to use.
	 */
	virtual FString GetBuildTarget() const = 0;
	
	/**
	 * Sets the default buid target to use.
	 */
	virtual void SetBuildTarget( const FString& InBuildTarget ) = 0;
	
	/**
	 * Return a list of all build build targets for the current project that need to be specified explicitly via -target=
	 * If a build target isn't in the list, it does not need to be specified (and BuildTarget can be an empty string)
	 */
	UE_DEPRECATED(5.8, "this was only used for the legacy project launcher")
	virtual const TArray<FString> GetAllExplicitBuildTargetNames() const = 0;

public:

	/**
	 * Returns a delegate that is invoked when a device group was added.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherProfileManagerDeviceGroupAdded& OnDeviceGroupAdded( ) = 0;

	/**
	 * Returns a delegate that is invoked when a device group was removed.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherProfileManagerDeviceGroupRemoved& OnDeviceGroupRemoved( ) = 0;

	/**
	 * Returns a delegate that is invoked when a profile was added.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherProfileManagerProfileAdded& OnProfileAdded( ) = 0;

	/**
	 * Returns a delegate that is invoked when a profile was removed.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherProfileManagerProfileRemoved& OnProfileRemoved( ) = 0;
	

	/**
	 * Returns a delegate that is invoked when the project is changed.
	 *
	 * @return The delegate.
	 */
	virtual FOnLauncherProfileManagerProjectChanged& OnProjectChanged( ) = 0;
	

	/**
	 * Returns a delegate that is invoked to post-process the command line before a profile launched
	 * 
	 * @return The delegate.
	 */
	virtual FOnLauncherProfilePostProcessCommandLine& OnPostProcessLaunchCommandLine( ) = 0;

	/**
	 * Returns a delegate that is invoked to post-process the command line before an automated test is launched
	 * 
	 * @return The delegate.
	 */
	virtual FOnLauncherProfilePostProcessAutomatedTestCommandLine& OnPostProcessAutomatedTestCommandLine( ) = 0;

	/**
	 * Returns a delegate that is invoked to post-process the command line before a UAT command is launched
	 * 
	 * @return The delegate.
	 */
	virtual FOnLauncherProfilePostProcessUATCommandLine& OnPostProcessUATCommandLine( ) = 0;

public:

	/** Virtual destructor. */
	virtual ~ILauncherProfileManager( ) = default;
};
