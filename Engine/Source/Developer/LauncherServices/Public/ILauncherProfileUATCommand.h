// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/JsonWriter.h"

class FJsonObject;
class ILauncherProfileAutomatedTest;
class ILauncherProfileBuildCookRun;

/**
 * UAT command parameters
 */
class ILauncherProfileUATCommand
{
public:
	//@todo: comments

	virtual void SetUATCommand( const TCHAR* UATCommand ) = 0;
	virtual const FString& GetUATCommand() const = 0;

	virtual void SetAdditionalUATCommandLine( const TCHAR* AdditionalCommandLine ) = 0;
	virtual const FString& GetAdditionalUATCommandLine() const = 0;

	virtual void SetEnabled( bool bEnabled ) = 0;
	virtual bool IsEnabled() const = 0;

	virtual void SetOrder( int32 Order ) = 0; // negative order items will run before the main BuildCookRun, positive order items will run afterwards
	virtual int32 GetOrder() const = 0;

	// recommendation for InternalName is is "ExtensionName.InternalName"
	virtual const TCHAR* GetInternalName() const = 0;

	virtual void SetDescription(const FString& NewDescription) = 0;
	virtual FString GetDescription() const = 0;

	virtual void SetUserTypeName(const FString& NewUserTypeName) = 0;
	virtual FString GetUserTypeName() const = 0;

	virtual bool Load(const FJsonObject& Object) = 0;
	virtual void Save(TJsonWriter<>& Writer) = 0;

	virtual TSharedPtr<ILauncherProfileAutomatedTest> AsAutomatedTest() = 0;
	virtual TSharedPtr<ILauncherProfileBuildCookRun> AsBuildCookRun() = 0;

	virtual ~ILauncherProfileUATCommand() = default;
};

typedef TSharedPtr<ILauncherProfileUATCommand> ILauncherProfileUATCommandPtr;
typedef TSharedRef<ILauncherProfileUATCommand> ILauncherProfileUATCommandRef;
