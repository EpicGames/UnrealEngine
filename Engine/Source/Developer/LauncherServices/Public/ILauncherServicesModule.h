// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ILauncherProfileManager.h"
#include "ILauncher.h"

/**
 * Declares a delegate to be invoked when a launcher profile manager has been initialized.
 *
 * The first parameter is the profile manager.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLauncherProfileManagerInitialized, ILauncherProfileManager&);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnCreateLauncher, ILauncherRef);


/**
 * Interface for launcher tools modules.
 */
class ILauncherServicesModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a new device group.
	 *
	 * @return A new device group.
	 */
	virtual ILauncherDeviceGroupRef CreateDeviceGroup( ) = 0;

	/**
	 * Creates a new device group.
	 *
	 * @return A new device group.
	 */
	virtual ILauncherDeviceGroupRef CreateDeviceGroup(const FGuid& Guid, const FString& Name  ) = 0;

	/**
	 * Creates a game launcher.
	 *
	 * @return A new game launcher.
	 */
	virtual ILauncherRef CreateLauncher( ) = 0;

	/**
	 * Creates a launcher profile.
	 *
	 * @param ProfileName - The name of the profile to create.
	 */
	virtual ILauncherProfileRef CreateProfile( const FString& ProfileName ) = 0;

	/**
	 * Gets the launcher profile manager.
	 *
	 * @return The launcher profile manager.
	 */
	virtual ILauncherProfileManagerRef GetProfileManager( ) = 0;
	
	/**
	 * Delegate for when a platform SDK isn't installed corrected (takes the platform name and the documentation link to show)
	 */
	DECLARE_EVENT_TwoParams(ILauncherServicesModule, FLauncherServicesSDKNotInstalled, const FString&, const FString&);
	virtual FLauncherServicesSDKNotInstalled& OnLauncherServicesSDKNotInstalled( ) = 0;
	virtual void BroadcastLauncherServicesSDKNotInstalled(const FString& PlatformName, const FString& DocLink) = 0;

	/**
	 * Determines the executable that should be used for executing commandlets
	 */
	virtual FString GetExecutableForCommandlets() const = 0;

public:

	/** Virtual destructor. */
	virtual ~ILauncherServicesModule( ) = default;

public:
	/**
	 * Delegate that is invoked when a profile manager is initialized.
	 */
	LAUNCHERSERVICES_API static FOnLauncherProfileManagerInitialized ProfileManagerInitializedDelegate;

	FOnCreateLauncher OnCreateLauncherDelegate;
};
