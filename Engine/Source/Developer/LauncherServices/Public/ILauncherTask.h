// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ILauncherTask;

namespace ELauncherTaskStatus
{
	/**
	 * Enumerates launcher task status types.
	 */
	enum Type
	{
		/** The task is currently busy executing. */
		Busy,

		/** The task has been canceled. */
		Canceled,

		/** The task completed successfully. */
		Completed,

		/** The task failed. */
		Failed,

		/** The task is waiting to execute. */
		Pending
	};
}


DECLARE_MULTICAST_DELEGATE_OneParam(FOnTaskStartedDelegate, const FString&);

/** Delegate used to notify when a stage ends */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTaskCompletedDelegate, const FString&);


/** Type definition for shared pointers to instances of ILauncherTask. */
typedef TSharedPtr<class ILauncherTask> ILauncherTaskPtr;

/** Type definition for shared references to instances of ILauncherTask. */
typedef TSharedRef<class ILauncherTask> ILauncherTaskRef;


/**
 * Interface for launcher worker tasks.
 */
class ILauncherTask
{
public:

	/**
	 * Cancels the task.
	 */
	virtual void Cancel( ) = 0;

	/**
	 * Gets the duration of time that the task has been running.
	 *
	 * @return Time duration.
	 */
	virtual FTimespan GetDuration( ) const = 0;

	/**
	 * Gets the task's name.
	 *
	 * @return Task name.
	 */
	virtual const FString& GetName( ) const = 0;

	/**
	 * Gets the task's name.
	 *
	 * @return Task name.
	 */
	virtual const FString& GetDesc( ) const = 0;

	/**
	 * Gets the task's current status.
	 *
	 * @return Task status.
	 */
	virtual ELauncherTaskStatus::Type GetStatus( ) const = 0;

	/**
	 * Gets if the task is trying to cancel
	 *
	 * @return if task is trying to cancel
	 */
	virtual bool IsCancelling() const = 0;


	/**
	 * Checks whether the task has finished execution.
	 *
	 * A task is finished when it is neither pending, nor busy.
	 *
	 * @return true if the task finished, false otherwise.
	 */
	virtual bool IsFinished( ) const = 0;

	/**
	 * retrieves the return code from the task
	 *
	 *
	 * @return return code from the task
	 */
	virtual int32 ReturnCode() const = 0;

	/**
	 * Gets the stage completed delegate
	 *
	 * @return the delegate
	 */
	virtual FOnTaskStartedDelegate& OnStarted() = 0;

	/**
	 * Gets the completed delegate
	 *
	 * @return the delegate
	 */
	virtual FOnTaskCompletedDelegate& OnCompleted() = 0;

	/**
	 * Gets the amount of errors that were generated during this task
	 *
	 * @return the amount
	 */
	virtual uint32 GetErrorCount() const = 0;

	/**
	 * Gets the amount of warnings that were generated during this task
	 *
	 * @return the amount
	 */
	virtual uint32 GetWarningCount() const = 0;
};
