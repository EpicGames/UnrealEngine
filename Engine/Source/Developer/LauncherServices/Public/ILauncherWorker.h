// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILauncherTask.h"
#include "ILauncherProfile.h"

class ILauncherWorker;

namespace ELauncherWorkerStatus
{
	enum Type
	{
		Busy,

		Canceling,

		Canceled,

		Completed
	};
}


/** Type definition for shared pointers to instances of ILauncherWorker. */
typedef TSharedPtr<class ILauncherWorker> ILauncherWorkerPtr;

/** Type definition for shared references to instances of ILauncherWorker. */
typedef TSharedRef<class ILauncherWorker> ILauncherWorkerRef;


/** Delegate used to notify of an output message */
DECLARE_MULTICAST_DELEGATE_OneParam(FOutputMessageReceivedDelegate, const FString&);

/** Delegate used to notify when a stage starts */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnStageStartedDelegate, const FString&);

/** Delegate used to notify when a stage ends */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStageCompletedDelegate, const FString&, double);

/** Delegate used to notify when the launch is complete */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLaunchCompletedDelegate, bool, double, int32);

/** Delegate used to notify when the launch was canceled */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLaunchCanceledDelegate, double);

/** For task list customization */
struct FCommandDesc
{
	FString Name;
	FString Desc;
	FString EndText;
};


/**
 * Interface for launcher worker threads.
 */
class ILauncherWorker
{
public:

	/**
	 * Cancels the operation.
	 */
	virtual void Cancel( ) = 0;

	/**
	 * Cancels the operation and waits for the thread to finish any remaining work.
	 */
	virtual void CancelAndWait( ) = 0;

	/**
	 * Gets the worker's status.
	 *
	 * @return Worker status.
	 */
	virtual ELauncherWorkerStatus::Type GetStatus( ) const = 0;

	/**
	 * Gets the worker's list of tasks.
	 *
	 * @param OutTasks - Will hold the task list.
	 *
	 * @return The number of tasks returned.
	 */
	virtual int32 GetTasks( TArray<ILauncherTaskPtr>& OutTasks ) const = 0;

	/**
	 * Gets the output message delegate
	 *
	 * @return the delegate
	 */
	virtual FOutputMessageReceivedDelegate& OnOutputReceived() = 0;

	/**
	 * Gets the stage started delegate
	 *
	 * @return the delegate
	 */
	virtual FOnStageStartedDelegate& OnStageStarted() = 0;

	/**
	 * Gets the stage completed delegate
	 *
	 * @return the delegate
	 */
	virtual FOnStageCompletedDelegate& OnStageCompleted() = 0;

	/**
	 * Gets the completed delegate
	 *
	 * @return the delegate
	 */
	virtual FOnLaunchCompletedDelegate& OnCompleted() = 0;

	/**
	 * Gets the canceled delegate
	 *
	 * @return the delegate
	 */
	virtual FOnLaunchCanceledDelegate& OnCanceled() = 0;

	/**
	 * Get the launcher profile
	 *
	 * @return the variable
	 */
	virtual ILauncherProfilePtr GetLauncherProfile() const = 0;

	/**
	 * Set the app id running on the device
	 */
	virtual void AddDevicePackagePair(const FString& Device, const FString& Package) {};

protected:

	/** Virtual destructor. */
	virtual ~ILauncherWorker( ) = default;
};
